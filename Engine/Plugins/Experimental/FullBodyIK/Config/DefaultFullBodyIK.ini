[CoreRedirects]
+PropertyRedirects=(OldName="RigUnit_FullbodyIK.EndEffectors",NewName="RigUnit_FullbodyIK.Effectors")

