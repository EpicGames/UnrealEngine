{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Full Body IK",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "FullBodyIK",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "PBIK",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	]
}