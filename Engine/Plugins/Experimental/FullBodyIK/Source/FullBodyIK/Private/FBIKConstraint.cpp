// Copyright Epic Games, Inc. All Rights Reserved.

#include "FBIKConstraint.h"
#include "FullBodyIK.h"
#include "TwoBoneIK.h"

bool FPositionLimitConstraint::ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<FTransform>& InOutLocalTransforms)
{
	if (ensure(InOutLinkData.IsValidIndex(ConstrainedIndex) && InOutLinkData.IsValidIndex(BaseIndex)))
	{
		FTransform LocalTransform = InOutLocalTransforms[ConstrainedIndex];

		bool bPositionChanged = false;

		auto SetLimitedPos = [&](float InDeltaPos, float InLimit) -> float
		{
			bPositionChanged = InDeltaPos < -InLimit || InDeltaPos > InLimit;
			return FMath::Clamp(InDeltaPos, -InLimit, InLimit);
		};

		FVector LocalPosition = LocalTransform.GetLocation();
		FVector LocalRefPosition = RelativelRefPose.GetLocation();

		if (bXLimitSet)
		{
			LocalPosition.X = SetLimitedPos(LocalPosition.X-LocalRefPosition.X, Limit.X);
		}

		if (bYLimitSet)
		{
			LocalPosition.Y = SetLimitedPos(LocalPosition.Y - LocalRefPosition.Y, Limit.Y);
		}

		if (bZLimitSet)
		{
			LocalPosition.Z = SetLimitedPos(LocalPosition.Z - LocalRefPosition.Z, Limit.Z);
		}

		if (bPositionChanged)
		{
			LocalTransform.SetLocation(LocalPosition);

			// update all links - @todo optimize
			InOutLocalTransforms[ConstrainedIndex] = LocalTransform;
			InOutLinkData[ConstrainedIndex].SetTransform(LocalTransform * InOutLinkData[BaseIndex].GetTransform());

			// calculate my new child local transform
			for (int32 ChildIndex = ConstrainedIndex + 1; ChildIndex < InOutLinkData.Num(); ++ChildIndex)
			{
				const int32 ParentIndex = InOutLinkData[ChildIndex].ParentLinkIndex;
				InOutLinkData[ChildIndex].SetTransform(InOutLocalTransforms[ChildIndex] * InOutLinkData[ParentIndex].GetTransform());
			}
		}

		return true;
	}

	return false;
}

bool FRotationLimitConstraint::ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<FTransform>& InOutLocalTransforms)
{
	if (ensure(InOutLinkData.IsValidIndex(ConstrainedIndex) && InOutLinkData.IsValidIndex(BaseIndex)))
	{
		FTransform LocalTransform = InOutLocalTransforms[ConstrainedIndex];
		// ref pose?
		// just do rotation for now
		FQuat LocalRotation = BaseFrameOffset.Inverse() * LocalTransform.GetRotation(); // child rotation
		LocalRotation.Normalize();
		FQuat LocalRefRotation = BaseFrameOffset.Inverse() * RelativelRefPose.GetRotation(); // later, maybe we'll make it more generic, so it doesn't always have to be just local transform
		LocalRefRotation.Normalize();
		bool bRotationChanged = false;

		auto SetLimitedQuat = [&](EAxis::Type InAxis, float InLimtAngle)
		{
			FQuat DeltaQuat = LocalRefRotation.Inverse() * LocalRotation;

			FVector RefTwistAxis = FMatrix::Identity.GetUnitAxis(InAxis);
			FQuat Twist, Swing;
			DeltaQuat.ToSwingTwist(RefTwistAxis, Swing, Twist);
			float SwingAngle = Swing.GetAngle();
			float TwistAngle = Twist.GetAngle();

			UE_LOGF(LogIKSolver, Log, "Delta Decomposition : Swing %ls (%f), Twist %ls (%f)",
				*Swing.GetRotationAxis().ToString(), FMath::RadiansToDegrees(SwingAngle),
				*Twist.GetRotationAxis().ToString(), FMath::RadiansToDegrees(TwistAngle));

			// we deal with the extra ones left
			LocalRotation = LocalRefRotation * Swing * FQuat(Twist.GetRotationAxis(), FMath::Clamp(TwistAngle, -InLimtAngle, InLimtAngle));
			LocalRotation.Normalize();
			bRotationChanged = true;
		};

		if (bXLimitSet)
		{
			SetLimitedQuat(EAxis::X, Limit.X);
		}

		if (bYLimitSet)
		{
			SetLimitedQuat(EAxis::Y, Limit.Y);
		}

		if (bZLimitSet)
		{
			SetLimitedQuat(EAxis::Z, Limit.Z);
		}


		if (bRotationChanged)
		{
			LocalRotation = BaseFrameOffset * LocalRotation;

			if ((LocalRotation | LocalTransform.GetRotation()) < 0.f)
			{
				LocalRotation *= -1.f;
			}

			LocalTransform.SetRotation(LocalRotation);
			LocalRotation.Normalize();

			// @Todo optimize
			InOutLocalTransforms[ConstrainedIndex] = LocalTransform;
			InOutLinkData[ConstrainedIndex].SetTransform(LocalTransform * InOutLinkData[BaseIndex].GetTransform());

			// calculate my new child local transform
			for (int32 ChildIndex = ConstrainedIndex + 1; ChildIndex < InOutLinkData.Num(); ++ChildIndex)
			{
				const int32 ParentIndex = InOutLinkData[ChildIndex].ParentLinkIndex;
				InOutLinkData[ChildIndex].SetTransform(InOutLocalTransforms[ChildIndex] * InOutLinkData[ParentIndex].GetTransform());
			}
		}

		return true;
	}

	return false;
}

FVector FPoleVectorConstraint::CalculateCurrentPoleVectorDir(const FTransform& ParentTransform, const FTransform& JointTransform, const FTransform& ChildTransform, const FQuat& LocalFrame) const
{
	// calculate in local space
	//FVector PoleDir = LocalFrame.RotateVector(PoleVectorDir);
	// calculate joint local transform
	FTransform JointLocalTransform = JointTransform.GetRelativeTransform(ParentTransform);
	// transform that local pole dir within the local transform
	FVector LocalTrDir = JointLocalTransform.TransformVector(PoleVector);
	// now find the target within local space
	FVector LocalJointTarget = JointLocalTransform.GetLocation() + LocalTrDir * 100.f; // extend
	// transform local target in parent space
	return ParentTransform.TransformPosition(LocalJointTarget);
}


bool FPoleVectorConstraint::ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<FTransform>& InOutLocalTransforms)
{
	if (ensure(InOutLinkData.IsValidIndex(BoneIndex) && InOutLinkData.IsValidIndex(ParentBoneIndex) && InOutLinkData.IsValidIndex(ChildBoneIndex)))
	{
		FTransform RootTransform = InOutLinkData[ParentBoneIndex].GetTransform();
		FTransform JointTransform = InOutLinkData[BoneIndex].GetTransform();
		FTransform ChildTransform = InOutLinkData[ChildBoneIndex].GetTransform();

		FVector JointTarget = (bUseLocalDir)? CalculateCurrentPoleVectorDir(RootTransform, JointTransform, ChildTransform, InOutLinkData[BoneIndex].LocalFrame) : PoleVector;
		
		AnimationCore::SolveTwoBoneIK(RootTransform, JointTransform, ChildTransform, JointTarget, ChildTransform.GetLocation(), false, 1.f, 1.f);

		auto UpdateTransform = [&](int32 LinkIndex, const FTransform& ParentTransform, const FTransform& InTransform)
		{
			InOutLocalTransforms[LinkIndex] = InTransform.GetRelativeTransform(ParentTransform);
			InOutLocalTransforms[LinkIndex].NormalizeRotation();

			InOutLinkData[LinkIndex].SetTransform(InTransform);
		};

		if (InOutLinkData[ParentBoneIndex].ParentLinkIndex != INDEX_NONE)
		{
			UpdateTransform(ParentBoneIndex, InOutLinkData[InOutLinkData[ParentBoneIndex].ParentLinkIndex].GetTransform(), RootTransform);
		}
		else
		{
			// this needs to be fixed 
			ensure(false);
			UpdateTransform(ParentBoneIndex, FTransform::Identity, RootTransform);
		}

		UpdateTransform(BoneIndex, RootTransform, JointTransform);
		UpdateTransform(ChildBoneIndex, JointTransform, ChildTransform);
		
		// calculate my new child local transform
		// we have to update everybody from ParentBoneIndex
		for (int32 ChildIndex = ParentBoneIndex + 1; ChildIndex < InOutLinkData.Num(); ++ChildIndex)
		{
			if (ChildIndex != BoneIndex && ChildIndex != ChildBoneIndex)
			{
				const int32 ParentIndex = InOutLinkData[ChildIndex].ParentLinkIndex;
				InOutLinkData[ChildIndex].SetTransform(InOutLocalTransforms[ChildIndex] * InOutLinkData[ParentIndex].GetTransform());
			}
		}

		return true;
	}

	return false;
}

