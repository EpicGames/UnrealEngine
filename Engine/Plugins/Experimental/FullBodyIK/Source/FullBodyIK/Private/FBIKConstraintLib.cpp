// Copyright Epic Games, Inc. All Rights Reserved.

#include "FBIKConstraintLib.h"
#include "FBIKConstraintOption.h"
#include "Misc/TVariantMeta.h"
#include "Rigs/RigHierarchy.h"

// in the future, we expose rotation axis 
namespace FBIKConstraintLib
{
	void ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<ConstraintType>* Constraints)
	{
		if (Constraints)
		{
			// apply partial delta to all joints
			TArray<FTransform> LocalTransforms;
			LocalTransforms.SetNum(InOutLinkData.Num());
			// before applying new delta transform, we save current local transform
			for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
			{
				// get the local translation first
				const int32 ParentIndex = InOutLinkData[LinkIndex].ParentLinkIndex;
				if (ParentIndex != INDEX_NONE)
				{
					LocalTransforms[LinkIndex] = InOutLinkData[LinkIndex].GetTransform().GetRelativeTransform(InOutLinkData[ParentIndex].GetTransform());
				}
				else
				{
					LocalTransforms[LinkIndex] = InOutLinkData[LinkIndex].GetTransform();
				}

				LocalTransforms[LinkIndex].NormalizeRotation();
			}

			for (int32 Index = 0; Index < Constraints->Num(); ++Index)
			{
				Visit([&](auto& Obj) { Obj.ApplyConstraint(InOutLinkData, LocalTransforms); }, (*Constraints)[Index]);
			}
		}
	}

	// for now allow linkdata to be modified here
	void BuildConstraints(const TArrayView<const FFBIKConstraintOption>& Constraints, TArray<ConstraintType>& OutConstraints,
		const URigHierarchy* Hierarchy, TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FRigElementKey>& LinkDataToHierarchyIndices,
		const TMap<FRigElementKey, int32>& HierarchyToLinkDataMap)
	{
		OutConstraints.Reset();

		for (int32 Index = 0; Index < Constraints.Num(); ++Index)
		{
			if (Constraints[Index].bEnabled)
			{
				const FRigElementKey& Item = Constraints[Index].Item;
				if (Item.IsValid())
				{
					// it's possible the joint exists, but not part of a solver
					// we don't handle that case yet. We can handle that outside of solver
					// @todo: think about this. Just general constraints can be done but then it doesn't have to be done per iteration?
					const int32* Found = HierarchyToLinkDataMap.Find(Item);
					if (Found)
					{
						int32 LinkIndex = *Found;

						auto GetLimit = [](EFBIKBoneLimitType Type, FVector::FReal DesiredLimit, FVector::FReal& OutLimit) -> bool
						{
							OutLimit = (Type == EFBIKBoneLimitType::Locked) ? 0.f : DesiredLimit;
							return (Type != EFBIKBoneLimitType::Free);
						};

						// for child index, we just find the first one
						if (Constraints[Index].IsAngularlyLimited())
						{
							// only allow when parent is within the link for now
							if (InOutLinkData[LinkIndex].ParentLinkIndex != INDEX_NONE)
							{
								const FRigElementKey& ParentItem = LinkDataToHierarchyIndices.FindChecked(InOutLinkData[LinkIndex].ParentLinkIndex);

								// add rotation limit
								FRotationLimitConstraint NewLimitConstraint;
								// for now we only constraint to parent
								NewLimitConstraint.BaseIndex = InOutLinkData[LinkIndex].ParentLinkIndex;
								NewLimitConstraint.ConstrainedIndex = LinkIndex; //*ChildLinkIndex;
								// if locked we should not add to solver for moving

								NewLimitConstraint.bXLimitSet = GetLimit(Constraints[Index].AngularLimit.LimitType_X, FMath::DegreesToRadians(Constraints[Index].AngularLimit.Limit.X), NewLimitConstraint.Limit.X);
								NewLimitConstraint.bYLimitSet = GetLimit(Constraints[Index].AngularLimit.LimitType_Y, FMath::DegreesToRadians(Constraints[Index].AngularLimit.Limit.Y), NewLimitConstraint.Limit.Y);
								NewLimitConstraint.bZLimitSet = GetLimit(Constraints[Index].AngularLimit.LimitType_Z, FMath::DegreesToRadians(Constraints[Index].AngularLimit.Limit.Z), NewLimitConstraint.Limit.Z);

								FTransform LocalTransform = Hierarchy->GetInitialGlobalTransform(Item).GetRelativeTransform(Hierarchy->GetInitialGlobalTransform(ParentItem));
								// @todo: think about 
								// set rotation frame
								NewLimitConstraint.BaseFrameOffset = FQuat(Constraints[Index].OffsetRotation);
								NewLimitConstraint.RelativelRefPose = LocalTransform;
								OutConstraints.Add(ConstraintType(TInPlaceType<FRotationLimitConstraint>(), NewLimitConstraint));
							}
						}

						// for child index, we just find the first one
						if (Constraints[Index].bUsePoleVector)
						{
							// only allow when parent is within the link for now
							if (InOutLinkData[LinkIndex].ParentLinkIndex != INDEX_NONE)
							{
								const FRigElementKey& ParentItem = LinkDataToHierarchyIndices.FindChecked(InOutLinkData[LinkIndex].ParentLinkIndex);

								// add rotation limit
								FPoleVectorConstraint PoleVectorConstraint;

								// for now we only constraint to parent
								PoleVectorConstraint.ParentBoneIndex = InOutLinkData[LinkIndex].ParentLinkIndex;
								PoleVectorConstraint.BoneIndex = LinkIndex; //*ChildLinkIndex;

								TArray<FRigElementKey> Children = Hierarchy->GetChildren(Item);
								for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
								{
									const int32* ChildLinkIndex = HierarchyToLinkDataMap.Find(Children[ChildIndex]);
									if (ChildLinkIndex)
									{
										PoleVectorConstraint.ChildBoneIndex = *ChildLinkIndex;
										PoleVectorConstraint.bUseLocalDir = Constraints[Index].PoleVectorOption == EPoleVectorOption::Direction;
										PoleVectorConstraint.PoleVector = Constraints[Index].PoleVector;
										if (PoleVectorConstraint.bUseLocalDir)
										{
											PoleVectorConstraint.PoleVector = PoleVectorConstraint.PoleVector.GetSafeNormal();
										}

										OutConstraints.Add(ConstraintType(TInPlaceType<FPoleVectorConstraint>(), PoleVectorConstraint));
										// we only add the first link. If you have multiple link, it may not what you want. 
										// we'll add the first one that we find from the link data
										break;
									}
								}
							}
						}

						InOutLinkData[LinkIndex].LocalFrame *= FQuat(Constraints[Index].OffsetRotation);
						InOutLinkData[LinkIndex].LocalFrame.Normalize();

						// add to motion bases for later
						InOutLinkData[LinkIndex].ResetMotionBases();

						if (Constraints[Index].bUseStiffness)
						{
							FMotionBase NewBase(FVector::ForwardVector);
							NewBase.SetAngularStiffness(Constraints[Index].AngularStiffness.X);
							NewBase.SetLinearStiffness(Constraints[Index].LinearStiffness.X);
							InOutLinkData[LinkIndex].AddMotionBase(NewBase);

							NewBase = FMotionBase(FVector::RightVector);
							NewBase.SetAngularStiffness(Constraints[Index].AngularStiffness.Y);
							NewBase.SetLinearStiffness(Constraints[Index].LinearStiffness.Y);
							InOutLinkData[LinkIndex].AddMotionBase(NewBase);

							NewBase = FMotionBase(FVector::UpVector);
							NewBase.SetAngularStiffness(Constraints[Index].AngularStiffness.Z);
							NewBase.SetLinearStiffness(Constraints[Index].LinearStiffness.Z);
							InOutLinkData[LinkIndex].AddMotionBase(NewBase);
						}
					}
				}
			}
		}
	}
}
