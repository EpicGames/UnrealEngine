// Copyright Epic Games, Inc. All Rights Reserved.

#include "FBIKShared.h"
#include "FBIKConstraint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FBIKShared)

void FJacobianSolver_FullbodyIK::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEffectors) const
{
	// since we're using constraints, we don't want to create new motion base all the time
	// constraints will add that info and we'll utilize it
}

void FJacobianSolver_FullbodyIK::PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEffectors) const
{
	// for final one, we'll have to add those in the beginning, and then just update the frame for every iteration
	for (int32 Index = 0; Index < InOutLinkData.Num(); ++Index)
	{
		// we update existing data
		const FQuat& LocalFrame = InOutLinkData[Index].LocalFrame;

		FQuat CurrentFrame;

		if (InOutLinkData[Index].ParentLinkIndex != INDEX_NONE)
		{
			CurrentFrame = InOutLinkData[InOutLinkData[Index].ParentLinkIndex].GetTransform().GetRotation() * LocalFrame;
		}
		else
		{
			CurrentFrame = LocalFrame;
		}

		// update the base axis
		if (InOutLinkData[Index].GetNumMotionBases() == 3)
		{
			InOutLinkData[Index].GetMotionBase(0).BaseAxis = CurrentFrame.RotateVector(FVector::ForwardVector);
			InOutLinkData[Index].GetMotionBase(1).BaseAxis = CurrentFrame.RotateVector(FVector::RightVector);
			InOutLinkData[Index].GetMotionBase(2).BaseAxis = CurrentFrame.RotateVector(FVector::UpVector);
		}
		else
		{
			ensure(InOutLinkData[Index].GetNumMotionBases() == 0);

			// invalid number of motion bases for this solver
			InOutLinkData[Index].ResetMotionBases();
			InOutLinkData[Index].AddMotionBase(FMotionBase(CurrentFrame.RotateVector(FVector::ForwardVector)));
			InOutLinkData[Index].AddMotionBase(FMotionBase(CurrentFrame.RotateVector(FVector::RightVector)));
			InOutLinkData[Index].AddMotionBase(FMotionBase(CurrentFrame.RotateVector(FVector::UpVector)));
		}
	}
}

