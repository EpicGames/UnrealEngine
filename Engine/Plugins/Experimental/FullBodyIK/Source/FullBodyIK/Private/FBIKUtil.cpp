// Copyright Epic Games, Inc. All Rights Reserved.

#include "FBIKUtil.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyDefines.h"

// in the future, we expose rotation axis 
namespace FBIKUtil
{
	bool CanCrossProduct(const FVector& Vector1, const FVector& Vector2)
	{
		// zero vectors produce NANs below
		if (Vector1.IsNearlyZero() || Vector2.IsNearlyZero())
		{
			return false;
		}
		// here we test their parallelism, and we normalize vectors
		float Cosine = FVector::DotProduct(Vector1.GetUnsafeNormal(), Vector2.GetUnsafeNormal());
		// if they're parallel, we need to find other axis
		return !(FMath::IsNearlyEqual(FMath::Abs(Cosine), 1.f));
	}

	FVector GetScaledRotationAxis(const FQuat& InQuat)
	{
		FVector Axis;
		float	Radian;

		InQuat.ToAxisAndAngle(Axis, Radian);

		Axis.Normalize();

		return Axis * Radian;
	}

	// the output should be from current index -> to root parent 
	bool GetBoneChain(URigHierarchy* Hierarchy, const FRigElementKey& Root, const FRigElementKey& Current, TArray<FRigElementKey>& ChainIndices)
	{
		ChainIndices.Reset();

		FRigElementKey Iterator = Current;

		// iterates until key is valid
		while (Iterator.IsValid() && Iterator != Root)
		{
			ChainIndices.Insert(Iterator, 0);
			Iterator = Hierarchy->GetFirstParent(Iterator);
		}

 		// add the last one if valid
 		if (Iterator.IsValid())
 		{
 			ChainIndices.Insert(Iterator, 0);
 		}

		// when you reached to something invalid, that means, we did not hit the 
		// expected root, we iterated to the root and above, and we exhausted our option
		// so if you hit or valid target by the hitting max depth, we should 
		return ChainIndices.Num() > 0;
	}
}
