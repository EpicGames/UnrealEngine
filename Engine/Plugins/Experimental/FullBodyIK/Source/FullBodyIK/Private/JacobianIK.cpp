// Copyright Epic Games, Inc. All Rights Reserved.

#include "JacobianIK.h"
#include "FBIKConstraint.h"
#include "FBIKUtil.h"

// in the future, we expose rotation axis 
namespace JacobianIK
{
	int32 CalculateColumnCount(const TArray<FFBIKLinkData>& InLinkData)
	{
		int32 ColCount = 0;

		for (int32 LinkIndex = 0; LinkIndex < InLinkData.Num(); ++LinkIndex)
		{
			// Add motion base for each column
			// if it allows that motion
			// first we add positional change of columns
			for (int32 AxisIndex = 0; AxisIndex < InLinkData[LinkIndex].GetNumMotionBases(); ++AxisIndex)
			{
				if (InLinkData[LinkIndex].IsLinearMotionAllowed(AxisIndex))
				{
					ColCount++;
				}
			}

			// second we add rotational change of columns
			for (int32 AxisIndex = 0; AxisIndex < InLinkData[LinkIndex].GetNumMotionBases(); ++AxisIndex)
			{
				if (InLinkData[LinkIndex].IsAngularMotionAllowed(AxisIndex))
				{
					ColCount++;
				}
			}
		}

		return ColCount;
	}

	int32 CalculateRowCount(const TMap<int32, FFBIKEffectorTarget>& InEndEffectors)
	{
		int32 RowCount = 0;
		for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
		{
			const FFBIKEffectorTarget& Target = Iter.Value();
			if (Target.bPositionEnabled)
			{
				++RowCount;
			}
			if (Target.bRotationEnabled)
			{
				++RowCount;
			}
		}

		return RowCount * 3;
	}

	// This is to compute partial derivatives for general rotation link to position target - 
	FVector ComputePositionalPartialDerivative(const FFBIKLinkData& InLinkData, const FFBIKLinkData& InEffectorLinkData, const FVector& RotationAxis)
	{
		FVector ToEffector = InEffectorLinkData.GetTransform().GetLocation() - InLinkData.GetTransform().GetLocation();
		// rotation axis doesn't have to be normalized
		// here Axis is rotation axis
        // now we want two of them
		// if not parallel, go for it
		if (FBIKUtil::CanCrossProduct(RotationAxis, ToEffector))
		{
			return FVector::CrossProduct(RotationAxis, ToEffector);
		}

		return FVector::ZeroVector;
	};

	void AllocateMatrix(Eigen::MatrixXf& InOutMatrix,
		const TArray<FFBIKLinkData>& InLinkData,
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors)
	{

		// each effector component uses 3 (for example position or rotation)
		const int32 NumEffectorElements = CalculateRowCount(InEndEffectors);
		const int32 NumLinkElements = CalculateColumnCount(InLinkData);
		// only allocate if the size differs
		if (InOutMatrix.rows() != NumEffectorElements || InOutMatrix.cols() != NumLinkElements)
		{
			InOutMatrix.resize(NumEffectorElements, NumLinkElements);
		}
	}

	bool CreateJacobianMatrix(const TArray<FFBIKLinkData>& InLinkData,
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		Eigen::MatrixXf& InOutMatrix,
		FCalculatePartialDerivativesDelegate OnCalculatePartialDerivatives, 
		const FSolverParameter& SolverParam)
	{
		check(OnCalculatePartialDerivatives.IsBound());

		AllocateMatrix(InOutMatrix, InLinkData, InEndEffectors);

		auto FillupPartialDerivativesForRow = [&](int32 InRowOffset, bool bInPositionTarget, const FFBIKEffectorTarget& InEffectorTarget, const FFBIKLinkData& InEffectorLinkData)
		{
			int32 ColOffset = 0;

			for (int32 LinkIndex = 0; LinkIndex < InLinkData.Num(); ++LinkIndex)
			{
				// if this effector cares for this link 
				if (InEffectorTarget.LinkChain.Contains(LinkIndex))
				{
					for (int32 LinkComponentIndex = 0; LinkComponentIndex < InLinkData[LinkIndex].GetNumMotionBases(); ++LinkComponentIndex)
					{
						if (InLinkData[LinkIndex].IsLinearMotionAllowed(LinkComponentIndex))
						{
							FVector PartialDerivative = OnCalculatePartialDerivatives.Execute(InLinkData[LinkIndex], true, LinkComponentIndex, InEffectorLinkData, bInPositionTarget, SolverParam);
							InOutMatrix(InRowOffset, ColOffset) = PartialDerivative.X;
							InOutMatrix(InRowOffset + 1, ColOffset) = PartialDerivative.Y;
							InOutMatrix(InRowOffset + 2, ColOffset) = PartialDerivative.Z;
							++ColOffset;
						}
					}
					for (int32 LinkComponentIndex = 0; LinkComponentIndex < InLinkData[LinkIndex].GetNumMotionBases(); ++LinkComponentIndex)
					{
						if (InLinkData[LinkIndex].IsAngularMotionAllowed(LinkComponentIndex))
						{
							// @todo: when we support posional(3 at a time), we may want to create different delegate
							FVector PartialDerivative = OnCalculatePartialDerivatives.Execute(InLinkData[LinkIndex], false, LinkComponentIndex, InEffectorLinkData, bInPositionTarget, SolverParam);
							InOutMatrix(InRowOffset, ColOffset) = PartialDerivative.X;
							InOutMatrix(InRowOffset + 1, ColOffset) = PartialDerivative.Y;
							InOutMatrix(InRowOffset + 2, ColOffset) = PartialDerivative.Z;
							++ColOffset;
						}
					}
				}
				else
				{
					for (int32 LinkComponentIndex = 0; LinkComponentIndex < InLinkData[LinkIndex].GetNumMotionBases(); ++LinkComponentIndex)
					{
						if (InLinkData[LinkIndex].IsLinearMotionAllowed(LinkComponentIndex))
						{
							InOutMatrix(InRowOffset, ColOffset) = 0.f;
							InOutMatrix(InRowOffset + 1, ColOffset) = 0.f;
							InOutMatrix(InRowOffset + 2, ColOffset) = 0.f;
							++ColOffset;
						}

					}

					for (int32 LinkComponentIndex = 0; LinkComponentIndex < InLinkData[LinkIndex].GetNumMotionBases(); ++LinkComponentIndex)
					{
						if (InLinkData[LinkIndex].IsAngularMotionAllowed(LinkComponentIndex))
						{
							InOutMatrix(InRowOffset, ColOffset) = 0.f;
							InOutMatrix(InRowOffset + 1, ColOffset) = 0.f;
							InOutMatrix(InRowOffset + 2, ColOffset) = 0.f;
							++ColOffset;
						}
					}
				}
			}

			ensure(ColOffset == InOutMatrix.cols());
		};

		if (InOutMatrix.rows() > 0 && InOutMatrix.cols() > 0)
		{
			int32 EffectorRowCount = 0;
			// rows
			for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
			{
				const int32 EffectorLinkIndex = Iter.Key();
				const FFBIKEffectorTarget& EffectorTarget = Iter.Value();
				const FFBIKLinkData& EffectorLinkData = InLinkData[EffectorLinkIndex];
				if (EffectorTarget.bPositionEnabled)
				{
					FillupPartialDerivativesForRow(EffectorRowCount * 3, true, EffectorTarget, EffectorLinkData);
					++EffectorRowCount;
				}
				if (EffectorTarget.bRotationEnabled)
				{
					FillupPartialDerivativesForRow(EffectorRowCount * 3, false, EffectorTarget, EffectorLinkData);
					++EffectorRowCount;
				}
			}

			ensure(EffectorRowCount * 3 == InOutMatrix.rows());
			return true;
		}

		return false;
	}

	bool CreateAnglePartialDerivativesUsingJPIDLS(const TArray<FFBIKLinkData>& InLinkData,
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		const Eigen::MatrixXf& InJacobianMatrix,
		Eigen::MatrixXf& InOutAngleDerivativeMatrix,
		FCalculateTargetVectorDelegate OnCalculateTargetVector,
		const FSolverParameter& SolverParam)
	{
		check(OnCalculateTargetVector.IsBound());

		// http://www.andreasaristidou.com/publications/papers/IK_survey.pdf
		// Calculate Jacobian Transpose
		Eigen::MatrixXf JacobianTransposeMatrix = InJacobianMatrix.transpose();
		// identity square matrix [# of effectors] * damping
		Eigen::MatrixXf DampingIdentityMatrix;
		const int32 NumEffectorRows = InJacobianMatrix.rows();

		// @todo: fix memory realloc
		DampingIdentityMatrix.resize(NumEffectorRows, NumEffectorRows);
		DampingIdentityMatrix.setIdentity();

		const float Damping = SolverParam.DampingValue;

		Eigen::MatrixXf JacobianSquare = InJacobianMatrix * JacobianTransposeMatrix; // rxr matrix
		JacobianSquare = JacobianSquare + Damping * Damping * DampingIdentityMatrix;
		if (!FMath::IsNearlyZero(JacobianSquare.determinant()))
		{
			Eigen::MatrixXf InverseMatrix = JacobianSquare.inverse();
			Eigen::MatrixXf EffectorDerivatives;
			EffectorDerivatives.resize(NumEffectorRows, 1);

			int32 EffectorRowCount = 0;
			for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
			{
				int32 EffectorLinkIndex = Iter.Key();
				const FFBIKEffectorTarget& EffectorTarget = Iter.Value();
				const FFBIKLinkData& EffectorLink = InLinkData[EffectorLinkIndex];

				if (EffectorTarget.bPositionEnabled)
				{
					FVector ToTarget = OnCalculateTargetVector.Execute(InLinkData[EffectorLinkIndex], EffectorTarget, true, SolverParam);
					const int32 RowOffset = (EffectorRowCount) * 3;
					EffectorDerivatives(RowOffset, 0) = ToTarget.X;
					EffectorDerivatives(RowOffset + 1, 0) = ToTarget.Y;
					EffectorDerivatives(RowOffset + 2, 0) = ToTarget.Z;
					++EffectorRowCount;
				}

				if (EffectorTarget.bRotationEnabled)
				{
					FVector ToTarget = OnCalculateTargetVector.Execute(InLinkData[EffectorLinkIndex], EffectorTarget, false, SolverParam);
					const int32 RowOffset = (EffectorRowCount) * 3;
					EffectorDerivatives(RowOffset, 0) = ToTarget.X;
					EffectorDerivatives(RowOffset + 1, 0) = ToTarget.Y;
					EffectorDerivatives(RowOffset + 2, 0) = ToTarget.Z;
					++EffectorRowCount;
				}
			}

			ensure(EffectorRowCount*3 == NumEffectorRows);

			InOutAngleDerivativeMatrix = JacobianTransposeMatrix * InverseMatrix * EffectorDerivatives;

			return true;
		}

		return false;
	}

	bool CreateAnglePartialDerivativesUsingJT(const TArray<FFBIKLinkData>& InLinkData,
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		const Eigen::MatrixXf& InJacobianMatrix,
		Eigen::MatrixXf& InOutAngleDerivativeMatrix,
		FCalculateTargetVectorDelegate OnCalculateTargetVector,
		const FSolverParameter& SolverParam)
	{

		check(OnCalculateTargetVector.IsBound());

		// Calculate Jacobian Transpose
		// http://www.andreasaristidou.com/publications/papers/IK_survey.pdf
		Eigen::MatrixXf JacobianTransposeMatrix = InJacobianMatrix.transpose();
		const int32 NumEffectorRows = InJacobianMatrix.rows();

		Eigen::MatrixXf JacobianSquare = InJacobianMatrix * JacobianTransposeMatrix; // rxr matrix

		Eigen::MatrixXf EffectorDerivatives;
		EffectorDerivatives.resize(NumEffectorRows, 1);

		int32 EffectorRowCount = 0;
		for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
		{
			int32 EffectorLinkIndex = Iter.Key();
			const FFBIKEffectorTarget& EffectorTarget = Iter.Value();
			const FFBIKLinkData& EffectorLink = InLinkData[EffectorLinkIndex];

			if (EffectorTarget.bPositionEnabled)
			{
				FVector ToTarget = OnCalculateTargetVector.Execute(InLinkData[EffectorLinkIndex], EffectorTarget, true, SolverParam);
				const int32 RowOffset = (EffectorRowCount) * 3;
				EffectorDerivatives(RowOffset, 0) = ToTarget.X;
				EffectorDerivatives(RowOffset + 1, 0) = ToTarget.Y;
				EffectorDerivatives(RowOffset + 2, 0) = ToTarget.Z;
				++EffectorRowCount;
			}

			if (EffectorTarget.bRotationEnabled)
			{
				FVector ToTarget = OnCalculateTargetVector.Execute(InLinkData[EffectorLinkIndex], EffectorTarget, false, SolverParam);
				const int32 RowOffset = (EffectorRowCount) * 3;
				EffectorDerivatives(RowOffset, 0) = ToTarget.X;
				EffectorDerivatives(RowOffset + 1, 0) = ToTarget.Y;
				EffectorDerivatives(RowOffset + 2, 0) = ToTarget.Z;
				++EffectorRowCount;
			}
		}

		ensure(EffectorRowCount*3 == NumEffectorRows);

		Eigen::VectorXf Result = JacobianSquare * EffectorDerivatives;
		Eigen::VectorXf EffectorDerivativesVector = EffectorDerivatives;
		float AlphaBottom = Result.dot(Result);
		float AlphaUp = EffectorDerivativesVector.dot(Result);

		if (!FMath::IsNearlyZero(AlphaBottom))
		{
			InOutAngleDerivativeMatrix = (AlphaUp / AlphaBottom) * JacobianTransposeMatrix * EffectorDerivatives;
			return true;
		}

		return false;
	}
}
