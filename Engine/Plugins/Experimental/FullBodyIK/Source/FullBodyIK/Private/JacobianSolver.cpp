// Copyright Epic Games, Inc. All Rights Reserved.
#include "JacobianSolver.h"
#include "FBIKConstraint.h"
#include "FBIKUtil.h"
#include "FullBodyIK.h"
#include "HAL/LowLevelMemTracker.h"

//////////////////////////////////////////////////////////////////
//// utility functions
/////////////////////////////////////////////////////////////////

void CalculateRotationAxisBasedOnEffectorPosition(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) 
{
	auto CalculateRotationAxis = [&](const FVector& ToEffector, const FVector& ToTarget) -> FVector
	{
		FVector RotationAxis;
		// find rotation axis with normalized (cross product of (target-current) ^ (effector-current)
		if (FBIKUtil::CanCrossProduct(ToEffector, ToTarget))
		{
			RotationAxis = FVector::CrossProduct(ToEffector, ToTarget);
		}
		else
		{
			// find better rotation axis if they all align
			if (FBIKUtil::CanCrossProduct(FVector::UpVector, ToTarget))
			{
				RotationAxis = FVector::CrossProduct(FVector::UpVector, ToTarget);
			}
			else
			{
				RotationAxis = FVector::CrossProduct(FVector::ForwardVector, ToTarget);
			}
		}

		RotationAxis.Normalize();

		return RotationAxis;
	};

	// calculate rotation axis before we created partial derivatives
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
		{
			const int32 EffectorLinkIndex = Iter.Key();
			const FFBIKEffectorTarget& Data = Iter.Value();

			// if this effector cares about this
			if (Data.LinkChain.Contains(LinkIndex))
			{
				FVector ToEffector = InOutLinkData[EffectorLinkIndex].GetTransform().GetLocation() - InOutLinkData[LinkIndex].GetTransform().GetLocation();
				FVector ToTarget = Data.Position - InOutLinkData[LinkIndex].GetTransform().GetLocation();

				// this motion is to apply angularly, not linearly since we're calculating to get to target by rotating joint
				FMotionBase MotionBase(CalculateRotationAxis(ToEffector, ToTarget));
				MotionBase.SetAngularStiffness(0.f);
				MotionBase.SetLinearStiffness(1.f);
				InOutLinkData[LinkIndex].AddMotionBase(MotionBase);
			}
		}

		// here we try to place all rotation axis in the same hemisphere, 
		const int32 NumMotionBases = InOutLinkData[LinkIndex].GetNumMotionBases();
		if (NumMotionBases > 0)
		{
			const FVector& FirstAxis = InOutLinkData[LinkIndex].GetRotationAxis(0);
			// after this, we try to place them in the same space, so to prevent flipping
			for (int32 MotionIndex = 1; MotionIndex < NumMotionBases; ++MotionIndex)
			{
				FVector TestAxis = InOutLinkData[LinkIndex].GetRotationAxis(MotionIndex);
				float DotProduct = FVector::DotProduct(FirstAxis, TestAxis);
				if (DotProduct < 0.f)
				{
					FMotionBase& CurrentBase = InOutLinkData[LinkIndex].GetMotionBase(MotionIndex);
					CurrentBase.BaseAxis *= -1.f;
				}
			}
		}
	}
}

void CalculateRotationAxisBasedOnEffectorRotation(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors)
{
	// calculate rotation axis before we created partial derivatives
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
		{
			const int32 EffectorLinkIndex = Iter.Key();
			const FFBIKEffectorTarget& Data = Iter.Value();

			// if this effector cares about this
			if (Data.LinkChain.Contains(LinkIndex))
			{
				FQuat ToTarget = InOutLinkData[EffectorLinkIndex].GetTransform().GetRotation().Inverse() * Data.Rotation;
				// this motion is to apply angularly, not linearly since we're calculating to get to target by rotating joint
				FMotionBase MotionBase(FBIKUtil::GetScaledRotationAxis(ToTarget));
				MotionBase.SetAngularStiffness(0.f);
				MotionBase.SetLinearStiffness(1.f);
				InOutLinkData[LinkIndex].AddMotionBase(MotionBase);
			}
		}
	}
}
////////////////////////////////////////////////////////////////////////////
///
// FJacobianSolverBase
// 
///////////////////////////////////////////////////////////////////////////////
FJacobianSolverBase::FJacobianSolverBase()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/FBIK"));
	OnCalculatePartialDerivativesDelegate = FCalculatePartialDerivativesDelegate::CreateRaw(this, &FJacobianSolverBase::ComputePartialDerivative);
	OnCalculateTargetVectorDelegate = FCalculateTargetVectorDelegate::CreateRaw(this, &FJacobianSolverBase::ComputeTargetVector);
}

// This is to compute partial derivatives for general rotation link to position target - 
FVector FJacobianSolverBase::ClampMag(const FVector& ToTargetVector, float Length) const
{
	float MaxLength = Length;
	if (ToTargetVector.SizeSquared() > MaxLength * MaxLength)
	{
		FVector ToTargetNorm = ToTargetVector.GetUnsafeNormal();
		return ToTargetNorm * MaxLength;
	}

	return ToTargetVector;
}

// This is to compute partial derivatives for general rotation link to position target - 
FVector FJacobianSolverBase::ComputePartialDerivative(const FFBIKLinkData& InLinkData, bool bPositionChange, int32 LinkComponentIdx, const FFBIKLinkData& InEffectorLinkData, bool bPositionTarget, const FSolverParameter& InSolverParameter)
{
	if (bPositionChange)
	{
		if (bPositionTarget)
		{
			// positional target, and this is moving linearly
			float Strength = InLinkData.GetLinearMotionStrength() * InLinkData.GetMotionBase(LinkComponentIdx).GetLinearMotionScale();
			return InLinkData.GetRotationAxis(LinkComponentIdx) * Strength;
		}
		else
		{
			// rotation target, and for rotation target, we don't move linearly
			// we don't do anything w.r.t. rotation
			return FVector::ZeroVector;
		}
	}
	else
	{
		// rotation axis
		FVector RotationAxis = InLinkData.GetRotationAxis(LinkComponentIdx);
		if (bPositionTarget)
		{
			// positional target and this is moving angularly
			float Strength = InLinkData.GetAngularMotionStrength() * InLinkData.GetMotionBase(LinkComponentIdx).GetAngularMotionScale();
			return JacobianIK::ComputePositionalPartialDerivative(InLinkData, InEffectorLinkData, RotationAxis) * Strength;
		}
		else
		{
			// rotational target and this is moving angularly
			float Strength = InLinkData.GetAngularMotionStrength() * InLinkData.GetMotionBase(LinkComponentIdx).GetAngularMotionScale();
			// if rotational target, we use rotaiton axis as partial derivatives
			return RotationAxis * Strength;
		}
	}
}

FVector FJacobianSolverBase::ComputeTargetVector(const FFBIKLinkData& InEffectorLink, const FFBIKEffectorTarget& InEffectorTarget, bool bPositionTarget, const FSolverParameter& SolverParam)
{
	if (bPositionTarget)
	{
		FVector ToTarget = (InEffectorTarget.Position - InEffectorLink.GetTransform().GetLocation()) /** InEffectorTarget.LinearMotionStrength*/;
		// this clamping may go to each link option?
		if (SolverParam.bClampToTarget)
		{
			ToTarget = ClampMag(ToTarget, InEffectorTarget.GetCurrentLength()/** (1.f- NormalizedIterationProgress)*/);
			//ToTarget = ClampMag(ToTarget, InEffectorTarget.GetCurrentLength()*InEffectorTarget.LinearMotionStrength*(0.2+0.8*(1.f-NormalizedIterationProgress)));
		}
		return ToTarget;
	}
	else
	{
		FQuat ToTarget = InEffectorTarget.Rotation * InEffectorLink.GetTransform().GetRotation().Inverse();
		FVector ToTargetVector = FBIKUtil::GetScaledRotationAxis(ToTarget);// * InEffectorTarget.AngularMotionStrength;

		return  ToTargetVector;
	}
}

bool FJacobianSolverBase::DidConverge(const TArray<FFBIKLinkData>& InLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors, float Tolerance, int32 Count) const
{
	// test if we converged? if so, break?
	bool bConvergedAll = true;
	for (auto Iter = InEndEffectors.CreateConstIterator(); Iter; ++Iter)
	{
		const int32 Index = Iter.Key();
		const FFBIKEffectorTarget& EffectorTarget = Iter.Value();
		if (EffectorTarget.bPositionEnabled)
		{
			FVector DiffVector = EffectorTarget.Position - InLinkData[Index].GetTransform().GetLocation();
			float Distance = DiffVector.Size();
			float ConvergeDistance = EffectorTarget.InitialPositionDistance * FMath::Clamp<float>(1.f - EffectorTarget.ConvergeScale, 0.f, 1.f);
			if (FMath::Abs(Distance - ConvergeDistance) > Tolerance)
			{
				bConvergedAll = false;
				UE_LOGF(LogIKSolver, Log, "Converge Location Test - Iteration Count : %d, LinkIndex : %d, Current Distance of %0.5f",
					Count, Index, Distance);
				break;
			}
		}

		// if converged, and if rotation is enabled, check it
		if (EffectorTarget.bRotationEnabled)
		{
			const FQuat Target = EffectorTarget.Rotation;
			const FQuat Effector = InLinkData[Index].GetTransform().GetRotation();
			FVector DiffVector = FBIKUtil::GetScaledRotationAxis(Target) - FBIKUtil::GetScaledRotationAxis(Effector);
			float Distance = DiffVector.Size();
			//float ConvergeDistance = EffectorTarget.InitialRotationDistance * FMath::Clamp<float>(1.f - Tolerance, 0.f, 1.f);
			if (FMath::Abs(Distance) > 0.1f)
			{
				bConvergedAll = false;
				UE_LOGF(LogIKSolver, Log, "Converge Rotation Test - Count : %d, LinkIndex : %d, Current Distance of %0.5f",
							Count, Index, Distance);
				break;
			}
		}

	}

	return bConvergedAll;
}

FQuat FJacobianSolverBase::GetDeltaRotation(const FFBIKLinkData& LinkData, int32& OutPartialDerivativesIndex) const
{
	FQuat Result = FQuat::Identity;

	for (int32 BaseIndex = 0; BaseIndex < LinkData.GetNumMotionBases(); ++BaseIndex)
	{
		const FMotionBase& MotionBase = LinkData.GetMotionBase(BaseIndex);
		if (MotionBase.IsAngularMotionAllowed())
		{
			Result = Result * FQuat(LinkData.GetRotationAxis(BaseIndex), AnglePartialDerivatives(OutPartialDerivativesIndex++, 0) * MotionBase.GetAngularMotionScale());
		}
	}

	return Result;
}

FVector FJacobianSolverBase::GetDeltaPosition(const FFBIKLinkData& LinkData, int32& OutPartialDerivativesIndex) const
{
	FVector Result = FVector::ZeroVector;

	for (int32 BaseIndex = 0; BaseIndex < LinkData.GetNumMotionBases(); ++BaseIndex)
	{
		const FMotionBase& MotionBase = LinkData.GetMotionBase(BaseIndex);
		if (MotionBase.IsLinearMotionAllowed())
		{
			Result = Result + LinkData.GetRotationAxis(BaseIndex) * AnglePartialDerivatives(OutPartialDerivativesIndex++, 0) * MotionBase.GetLinearMotionScale();
		}
	}

	return Result;
}

bool FJacobianSolverBase::RunJacobianIK(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors, const FSolverParameter& InSolverParameter, float NormalizedIterationProgress)
{
	// we do this for debugging
	// for now we disabled this as we don't use it
	for (int32 Index = 0; Index < InOutLinkData.Num(); ++Index)
	{
		InOutLinkData[Index].SavePreviousTransform();
	}

	PreSolve(InOutLinkData, InEndEffectors);

	// each frame, we have to recalculate something 
	if (CreateJacobianMatrix(InOutLinkData, InEndEffectors, JacobianMatrix,
		OnCalculatePartialDerivativesDelegate, InSolverParameter))
	{
		bool bSuccess = false;
		
		if (InSolverParameter.JacobianSolver == EJacobianSolver::JacobianTranspose)
		{
			bSuccess = CreateAnglePartialDerivativesUsingJT(InOutLinkData, InEndEffectors,
				JacobianMatrix, AnglePartialDerivatives,
				OnCalculateTargetVectorDelegate,
				InSolverParameter);
		}
		else
		{
			bSuccess = CreateAnglePartialDerivativesUsingJPIDLS(InOutLinkData, InEndEffectors,
				JacobianMatrix, AnglePartialDerivatives,
				OnCalculateTargetVectorDelegate,
				InSolverParameter);
		}

		if (bSuccess)
		{
			UpdateLinkData(InOutLinkData, AnglePartialDerivatives);
			return true;
		}
	}

	return false;
}

void FJacobianSolverBase::UpdateLinkData(TArray<FFBIKLinkData>& InOutLinkData, const Eigen::MatrixXf& InAnglePartialDerivatives) const
{
	// apply partial delta to all joints
	TArray<FTransform> LocalTransforms;
	LocalTransforms.SetNum(InOutLinkData.Num());
	// before applying new delta transform, we save current local transform
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		// get the local translation first
		const int32 ParentIndex = InOutLinkData[LinkIndex].ParentLinkIndex;
		if (ParentIndex != INDEX_NONE)
		{
			LocalTransforms[LinkIndex] = InOutLinkData[LinkIndex].GetTransform().GetRelativeTransform(InOutLinkData[ParentIndex].GetTransform());
		}
		else
		{
			LocalTransforms[LinkIndex] = InOutLinkData[LinkIndex].GetTransform();
		}

		LocalTransforms[LinkIndex].NormalizeRotation();
	}

	// apply to inoutlink data
	int32 PartialDerivatiesOffset = 0;
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		FFBIKLinkData& LinkData = InOutLinkData[LinkIndex];
		FTransform LinkTransform = InOutLinkData[LinkIndex].GetTransform();

		// if all axes are locked, the GetDeltaPosition should return 0, and it won't change anything, 
		// but we may want to optimize or validate that.  @todo: LH
		// apply position
		{
			FVector NewPosition = GetDeltaPosition(InOutLinkData[LinkIndex], PartialDerivatiesOffset);
			FVector TargetPosition = NewPosition + LinkTransform.GetLocation();
			LinkTransform.SetLocation(TargetPosition);
		}

		// if all axes are locked, the GetDeltaPosition should return 0, and it won't change anything, 
		// but we may want to optimize or validate that.  @todo: LH
		// apply rotation change
		{
			FQuat NewRotation = GetDeltaRotation(InOutLinkData[LinkIndex], PartialDerivatiesOffset);
			FQuat TargetRotation = NewRotation * LinkTransform.GetRotation();
			LinkTransform.SetRotation(TargetRotation);
			LinkTransform.NormalizeRotation();
		}
		
		LinkData.SetTransform(LinkTransform);		

		// calculate my new child local transform
		for (int32 ChildIndex = LinkIndex + 1; ChildIndex < InOutLinkData.Num(); ++ChildIndex)
		{
			const int32 ParentIndex = InOutLinkData[ChildIndex].ParentLinkIndex;
			ensure(ParentIndex != INDEX_NONE);
			InOutLinkData[ChildIndex].SetTransform(LocalTransforms[ChildIndex] * InOutLinkData[ParentIndex].GetTransform());
		}
	}

	ensure(PartialDerivatiesOffset == InAnglePartialDerivatives.rows());

	// this can be used for post process of the pose
	// I'm experimenting with applying constraint as while we solve pose or not
	// but as Jacobian is a black box, and there is nothing much we can do after the execution
	// I don't see any good reason on the applying while we're applying to each bone
	// so for now I'm testing with one post process execution to see how this works
	OnPostProcessDelegateForIteration.ExecuteIfBound(InOutLinkData);
}

bool FJacobianSolverBase::SolveJacobianIK(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors, 
	const FSolverParameter& InSolverParameter, int32 IterationCount /*= 30*/, float Tolerance /*= 1.f*/, TArray<FJacobianDebugData>* DebugData /*= nullptr*/)
{
	// when we use bUpdateClampMagnitude, we make a copy b/c we modify it internally
	// don't want to return this value
	TMap<int32, FFBIKEffectorTarget> EndEffectors = InEndEffectors;
	// prepare to run
	InitializeSolver(InOutLinkData, EndEffectors);

	// if update clamp magnitude is on, clamp it
	if (InSolverParameter.bUpdateClampMagnitude)
	{
		// if update clamp is true, make sure to set big number for adjusted
		for (auto Iter = EndEffectors.CreateIterator(); Iter; ++Iter)
		{
			FFBIKEffectorTarget& Data = Iter.Value();
			// make it 10 times bigger than current length
			// in the paper this number is infinite
			Data.AdjustedLength = Data.ChainLength * 10.f;
		}
	}

	// this is iteration loop
	for (int32 Count = 0; Count < IterationCount; ++Count)
	{
		// run jacobian
		float NormalizedIterationProgress = (float)Count / (float)IterationCount;
		if (RunJacobianIK(InOutLinkData, EndEffectors, InSolverParameter, NormalizedIterationProgress))
		{
			if (DebugData)
			{
				int32 DebugDataIndex = DebugData->AddZeroed();
				(*DebugData)[DebugDataIndex].LinkData = InOutLinkData;
				// we only do this if position is active
				(*DebugData)[DebugDataIndex].TargetVectorSources.Reset();
				(*DebugData)[DebugDataIndex].TargetVectors.Reset();
				for (auto Iter = EndEffectors.CreateConstIterator(); Iter; ++Iter)
				{
					const FFBIKEffectorTarget& EffectorTarget = Iter.Value();
					if (EffectorTarget.bPositionEnabled)
					{
							const FFBIKLinkData& EffectorLink = InOutLinkData[Iter.Key()];
							// source is effector target link index - we need pre-solve transform
							(*DebugData)[DebugDataIndex].TargetVectorSources.Add(EffectorLink.GetPreviousTransform());
							(*DebugData)[DebugDataIndex].TargetVectors.Add(ComputeTargetVector(EffectorLink, EffectorTarget, true, InSolverParameter));
					}
				}
			}

			if (DidConverge(InOutLinkData, EndEffectors, Tolerance, Count))
			{
				return true;
			}

			if (InSolverParameter.bUpdateClampMagnitude)
			{
				UpdateClampMag(InOutLinkData, EndEffectors);
			}
		}
	}

	return false;
}


void FJacobianSolverBase::UpdateClampMag(const TArray<FFBIKLinkData>& InLinkData, TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const
{
	// test if we converged? if so, break?
	for (auto Iter = InEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		const int32 Index = Iter.Key();
		FFBIKEffectorTarget& EffectorTarget = Iter.Value();
		if (EffectorTarget.bPositionEnabled)
		{
			const FVector& SourceLocation = InLinkData[Index].GetPreviousTransform().GetLocation();
			const FVector& NewLocation = InLinkData[Index].GetTransform().GetLocation();
			float PreviousDistanceToEffector = (EffectorTarget.Position - SourceLocation).Size();
			float NewDistanceToEffector = (EffectorTarget.Position - NewLocation).Size();

			// to do - another end condition to be implemented
			// local minima
			// did I get closer?
	//			if (PreviousDistanceToEffector > NewDistanceToEffector)
			{
				float DeltaMovement = (NewLocation - SourceLocation).Size();
				EffectorTarget.AdjustedLength = DeltaMovement;
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolver_PositionTarget_3DOF
//
//////////////////////////////////////////////////////////////////////////////////////////
FJacobianSolver_PositionTarget_3DOF::FJacobianSolver_PositionTarget_3DOF()
	: FJacobianSolverBase()
{}

void FJacobianSolver_PositionTarget_3DOF::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	// update twist info 
	// for now we disabled this as we don't use it
	for (int32 Index = 0; Index < InOutLinkData.Num(); ++Index)
	{
		// 3d supports 3 axis
		InOutLinkData[Index].ResetMotionBases();
		InOutLinkData[Index].AddMotionBase(FMotionBase(FVector::ForwardVector));
		InOutLinkData[Index].AddMotionBase(FMotionBase(FVector::RightVector));
		InOutLinkData[Index].AddMotionBase(FMotionBase(FVector::UpVector));
	}

	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = true;
		Target.bRotationEnabled = false;
	}
}
//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolver_PositionTarget_Quat
//
///////////////////////////////////////////////////////////////////////////////////////////

FJacobianSolver_PositionTarget_Quat::FJacobianSolver_PositionTarget_Quat()
	: FJacobianSolverBase()
{

}

void FJacobianSolver_PositionTarget_Quat::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = true;
		Target.bRotationEnabled = false;
	}
}

void FJacobianSolver_PositionTarget_Quat::PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const
{
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		InOutLinkData[LinkIndex].ResetMotionBases();
	}

	CalculateRotationAxisBasedOnEffectorPosition(InOutLinkData, InEndEffectors);
}

//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolver_RotationTarget_Quat
//
///////////////////////////////////////////////////////////////////////////////////////////

FJacobianSolver_RotationTarget_Quat::FJacobianSolver_RotationTarget_Quat()
	: FJacobianSolverBase()
{
}

void FJacobianSolver_RotationTarget_Quat::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	FJacobianSolverBase::InitializeSolver(InOutLinkData, InOutEndEffectors);

	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = false;
		Target.bRotationEnabled = true;
	}
}

void FJacobianSolver_RotationTarget_Quat::PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const
{
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		InOutLinkData[LinkIndex].ResetMotionBases();
	}

	CalculateRotationAxisBasedOnEffectorRotation(InOutLinkData, InEndEffectors);
}

//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolverRotationTarget3D
//
///////////////////////////////////////////////////////////////////////////////////////////

FJacobianSolver_RotationTarget_3DOF::FJacobianSolver_RotationTarget_3DOF()
	: FJacobianSolver_PositionTarget_3DOF()
{
}

void FJacobianSolver_RotationTarget_3DOF::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	FJacobianSolver_PositionTarget_3DOF::InitializeSolver(InOutLinkData, InOutEndEffectors);

	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = false;
		Target.bRotationEnabled = true;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolverPositionalRotationalTarget_3DOF
//
///////////////////////////////////////////////////////////////////////////////////////////

FJacobianSolver_PositionRotationTarget_3DOF::FJacobianSolver_PositionRotationTarget_3DOF()
	: FJacobianSolver_PositionTarget_3DOF()
{
}

void FJacobianSolver_PositionRotationTarget_3DOF::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	FJacobianSolver_PositionTarget_3DOF::InitializeSolver(InOutLinkData, InOutEndEffectors);

	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = true;
		Target.bRotationEnabled = true;
	}
}


//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolverPositionalRotationalTarget_Quat
//
///////////////////////////////////////////////////////////////////////////////////////////

FJacobianSolver_PositionRotationTarget_Quat::FJacobianSolver_PositionRotationTarget_Quat()
	: FJacobianSolver_PositionTarget_Quat()
{
}

void FJacobianSolver_PositionRotationTarget_Quat::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = true;
		Target.bRotationEnabled = true;
	}
}

void FJacobianSolver_PositionRotationTarget_Quat::PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const
{
	for (int32 LinkIndex = 0; LinkIndex < InOutLinkData.Num(); ++LinkIndex)
	{
		InOutLinkData[LinkIndex].ResetMotionBases();
	}

	CalculateRotationAxisBasedOnEffectorPosition(InOutLinkData, InEndEffectors);
	CalculateRotationAxisBasedOnEffectorRotation(InOutLinkData, InEndEffectors);
}

//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolver_PositionTarget_3DOF_Translation
//
///////////////////////////////////////////////////////////////////////////////////////////

FJacobianSolver_PositionTarget_3DOF_Translation::FJacobianSolver_PositionTarget_3DOF_Translation()
	: FJacobianSolver_PositionTarget_3DOF()
{
}

void FJacobianSolver_PositionTarget_3DOF_Translation::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	FJacobianSolver_PositionTarget_3DOF::InitializeSolver(InOutLinkData, InOutEndEffectors);

	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = true;
		Target.bRotationEnabled = false;
	}

	for (FFBIKLinkData& LinkData : InOutLinkData)
	{
		for (int32 BaseIdx = 0; BaseIdx < LinkData.GetNumMotionBases(); ++BaseIdx)
		{
			// we allow linear motion here
			FMotionBase& MotionBase = LinkData.GetMotionBase(BaseIdx);
			MotionBase.SetLinearStiffness(0.f);
			MotionBase.SetAngularStiffness(1.f);
	
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////
//
//	FJacobianSolver_PositionRotationTarget_LocalFrame - custom motion bases that targets rotation/position
// it won't work unless you add your motion bases manually
//
///////////////////////////////////////////////////////////////////////////////////////////
FJacobianSolver_PositionRotationTarget_LocalFrame::FJacobianSolver_PositionRotationTarget_LocalFrame()
	: FJacobianSolverBase()
{
}

void FJacobianSolver_PositionRotationTarget_LocalFrame::InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const
{
	// update twist info 
// 	// we add initial solvers 
// 	for (int32 Index = 0; Index < InOutLinkData.Num(); ++Index)
// 	{
// 		// 3d supports 3 axis
// 		InOutLinkData[Index].ResetMotionBases();
// 		InOutLinkData[Index].AddMotionBase(FMotionBase(FVector::ForwardVector));
// 		InOutLinkData[Index].AddMotionBase(FMotionBase(FVector::RightVector));
// 		InOutLinkData[Index].AddMotionBase(FMotionBase(FVector::UpVector));
// 	}

	for (auto Iter = InOutEndEffectors.CreateIterator(); Iter; ++Iter)
	{
		FFBIKEffectorTarget& Target = Iter.Value();
		Target.bPositionEnabled = true;
		Target.bRotationEnabled = true;
	}
}

void FJacobianSolver_PositionRotationTarget_LocalFrame::PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const
{
	// for final one, we'll have to add those in the beginning, and then just update the frame for every iteration
	for (int32 Index = 0; Index < InOutLinkData.Num(); ++Index)
	{
		// we update existing data
		const FQuat& LocalFrame = InOutLinkData[Index].LocalFrame;

		FQuat CurrentFrame;

		if (InOutLinkData[Index].ParentLinkIndex != INDEX_NONE)
		{
			CurrentFrame = InOutLinkData[InOutLinkData[Index].ParentLinkIndex].GetTransform().GetRotation() * LocalFrame;
		}
		else
		{
			CurrentFrame = LocalFrame;
		}

		// update the base axis
		if (InOutLinkData[Index].GetNumMotionBases() == 3)
		{
			InOutLinkData[Index].GetMotionBase(0).BaseAxis = CurrentFrame.RotateVector(FVector::ForwardVector);
			InOutLinkData[Index].GetMotionBase(1).BaseAxis = CurrentFrame.RotateVector(FVector::RightVector);
			InOutLinkData[Index].GetMotionBase(2).BaseAxis = CurrentFrame.RotateVector(FVector::UpVector);
		}
		else
		{
			ensure(InOutLinkData[Index].GetNumMotionBases() == 0);

			// invalid number of motion bases for this solver
			InOutLinkData[Index].ResetMotionBases();
			InOutLinkData[Index].AddMotionBase(FMotionBase(CurrentFrame.RotateVector(FVector::ForwardVector)));
			InOutLinkData[Index].AddMotionBase(FMotionBase(CurrentFrame.RotateVector(FVector::RightVector)));
			InOutLinkData[Index].AddMotionBase(FMotionBase(CurrentFrame.RotateVector(FVector::UpVector)));
		}
	}
}
