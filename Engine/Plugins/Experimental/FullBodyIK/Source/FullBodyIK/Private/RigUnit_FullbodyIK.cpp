// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_FullbodyIK.h"
#include "FBIKUtil.h"
#include "FBIKConstraintLib.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_FullbodyIK)

#define MAX_DEPTH 10000
/////////////////////////////////////////////////////

/////////////////////////////////////////////////////


static float EnsureToAddBoneToLinkData(URigHierarchy* Hierarchy, const FRigElementKey& CurrentItem, TArray<FFBIKLinkData>& LinkData,
	TMap<FRigElementKey, int32>& HierarchyToLinkDataMap, TMap<int32, FRigElementKey>& LinkDataToHierarchyIndices)
{
	float ChainLength = 0;

	// we insert from back to first, but only if the list doesn't have it
	int32* FoundLinkIndex = HierarchyToLinkDataMap.Find(CurrentItem);
	if (!FoundLinkIndex)
	{
		int32 NewLinkIndex = LinkData.AddDefaulted();
		FFBIKLinkData& NewLink = LinkData[NewLinkIndex];

		// find parent LinkIndex
		FRigElementKey ParentItem = Hierarchy->GetFirstParent(CurrentItem);

		const int32 CurrentItemIndex = Hierarchy->GetIndex(CurrentItem);
		const int32 ParentItemIndex = Hierarchy->GetIndex(ParentItem); 
		
		FoundLinkIndex = HierarchyToLinkDataMap.Find(ParentItem);
		NewLink.ParentLinkIndex = (FoundLinkIndex) ? *FoundLinkIndex : INDEX_NONE;
		NewLink.SetTransform(Hierarchy->GetGlobalTransform(CurrentItem));

		if (ParentItem.IsValid())
		{
			FVector DiffLocation = Hierarchy->GetInitialGlobalTransform(CurrentItemIndex).GetLocation() - Hierarchy->GetInitialGlobalTransform(ParentItemIndex).GetLocation();
			// set Length
			NewLink.Length = DiffLocation.Size();
		}

		// we create bidirectional look up table
		HierarchyToLinkDataMap.Add(CurrentItem, NewLinkIndex);
		LinkDataToHierarchyIndices.Add(NewLinkIndex, CurrentItem);

		ChainLength = NewLink.Length;
	}
	else
	{
		ChainLength = LinkData[*FoundLinkIndex].Length;
	}

	return ChainLength;
}

static void AddToEffectorTarget(int32 EffectorIndex, const FRigElementKey& Effector, TMap<int32, FFBIKEffectorTarget>& EffectorTargets, const TMap<FRigElementKey, int32>& HierarchyToLinkDataMap,
	TArray<int32>& EffectorLinkIndices,	float ChainLength, const TArray<FRigElementKey>& ChainIndices, int32 PositionDepth, int32 RotationDepth)
{
	const int32* EffectorLinkIndexID = HierarchyToLinkDataMap.Find(Effector);
	check(EffectorLinkIndexID);
	EffectorLinkIndices[EffectorIndex] = *EffectorLinkIndexID;
	// add EffectorTarget for this link Index
	FFBIKEffectorTarget& EffectorTarget = EffectorTargets.FindOrAdd(*EffectorLinkIndexID);
	EffectorTarget.ChainLength = ChainLength;

	// convert bone chain indices to link chain
	const int32 MaxNum = FMath::Max(PositionDepth, RotationDepth);
	if (ensure(MaxNum <= ChainIndices.Num()))
	{
		EffectorTarget.LinkChain.Reset(MaxNum);
		for (int32 Index = 0; Index< MaxNum; ++Index)
		{
			const FRigElementKey& Bone = ChainIndices[Index];
			const int32* LinkIndexID = HierarchyToLinkDataMap.Find(Bone);
			EffectorTarget.LinkChain.Add(*LinkIndexID);
		}
	}
}

static void AddEffectors(URigHierarchy* Hierarchy, const FRigElementKey& Root, const TArrayView<const FFBIKEndEffector>& Effectors,
	TArray<FFBIKLinkData>& LinkData, TMap<int32, FFBIKEffectorTarget>& EffectorTargets, TArray<int32>& EffectorLinkIndices, 
	TMap<int32, FRigElementKey>& LinkDataToHierarchyIndices, TMap<FRigElementKey, int32>& HierarchyToLinkDataMap, const FSolverInput& SolverProperty)
{
	EffectorLinkIndices.SetNum(Effectors.Num());
	// fill up all effector indices
	for (int32 Index = 0; Index < Effectors.Num(); ++Index)
	{
		// clear link indices, so that we don't search
		EffectorLinkIndices[Index] = INDEX_NONE;
		// create LinkeData from root bone to all effectors 
		const FRigElementKey& Item = Effectors[Index].Item;
		if (!Item.IsValid())
		{
			continue;
		}

		// can only add IK to bones
		if (Item.Type != ERigElementType::Bone)
		{
			continue;
		}

		const FFBIKEndEffector& CurrentEffector = Effectors[Index];

		TArray<FRigElementKey> ChainIndices;
		// if we haven't got to root, this is not valid chain
		if (FBIKUtil::GetBoneChain(Hierarchy, Root, Item, ChainIndices))
		{
			auto CalculateStrength = [&](int32 InBoneChainDepth, const int32 MaxDepth, float CurrentStrength, float MinStrength) -> float
			{
				const float Range = FMath::Max(CurrentStrength - MinStrength, 0.f);
				const float ApplicationStrength = (float)(1.f - (float)InBoneChainDepth / (float)MaxDepth) * Range;
				return ApplicationStrength + MinStrength;
				//return FMath::Clamp(ApplicationStrength + MinStrength, MinStrength, CurrentStrength);
			};

			auto UpdateMotionStrength = [&](int32 InBoneChainDepth,
				const int32 MaxPositionDepth, const int32 MaxRotationDepth, FFBIKLinkData& InOutNewLink)
			{
				// add motion scales
				float LinearMotionStrength;
				float AngularMotionStrength;

				if (CurrentEffector.PositionDepth <= InBoneChainDepth)
				{
					LinearMotionStrength = 0.f;
				}
				else
				{
					LinearMotionStrength = CalculateStrength(InBoneChainDepth, MaxPositionDepth, SolverProperty.LinearMotionStrength, SolverProperty.MinLinearMotionStrength);
				}

				if (CurrentEffector.RotationDepth <= InBoneChainDepth)
				{
					AngularMotionStrength = 0.f;
				}
				else
				{
					AngularMotionStrength = CalculateStrength(InBoneChainDepth, MaxRotationDepth, SolverProperty.AngularMotionStrength, SolverProperty.MinAngularMotionStrength);
				}

				InOutNewLink.AddMotionStrength(LinearMotionStrength, AngularMotionStrength);
			};

			// position depth and rotation depth can't go beyond of it
			// for now we cull it. 
			const int32 PositionDepth = FMath::Min(CurrentEffector.PositionDepth, ChainIndices.Num());
			const int32 RotationDepth = FMath::Min(CurrentEffector.RotationDepth, ChainIndices.Num());

			float ChainLength = 0.f;

			// add to link data
			for (int32 BoneChainIndex = 0; BoneChainIndex < ChainIndices.Num(); ++BoneChainIndex)
			{
				const FRigElementKey& CurrentItem = ChainIndices[BoneChainIndex];
				ChainLength += EnsureToAddBoneToLinkData(Hierarchy, CurrentItem, LinkData,
					HierarchyToLinkDataMap, LinkDataToHierarchyIndices);

				const int32 ChainDepth = ChainIndices.Num() - BoneChainIndex;
				int32* FoundLinkIndex = HierarchyToLinkDataMap.Find(CurrentItem);

				// now we should always have it
				check(FoundLinkIndex);
				UpdateMotionStrength(ChainDepth, PositionDepth, RotationDepth, LinkData[*FoundLinkIndex]);
			}

			// add to EffectorTargets
			AddToEffectorTarget(Index, Item, EffectorTargets, HierarchyToLinkDataMap, EffectorLinkIndices, ChainLength, ChainIndices, PositionDepth, RotationDepth);
		}
	}
}

/////////////////////////////////////////////////////

FRigUnit_FullbodyIK_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	// workdata reference
	TArray<FFBIKLinkData>& LinkData = WorkData.LinkData;
	TMap<int32, FFBIKEffectorTarget>& EffectorTargets = WorkData.EffectorTargets;
	TArray<int32>& EffectorLinkIndices = WorkData.EffectorLinkIndices;
	TMap<int32, FRigElementKey>& LinkDataToHierarchyIndices = WorkData.LinkDataToHierarchyIndices;
	TMap<FRigElementKey, int32>& HierarchyToLinkDataMap = WorkData.HierarchyToLinkDataMap;
	TArray<ConstraintType>& InternalConstraints = WorkData.InternalConstraints;

	// during iteration, (editortime), we allow them to modify effector count and chain joint length and so on
	if (Root.IsValid() 
#if !WITH_EDITOR
		&& LinkDataToHierarchyIndices.Num() == 0
#endif 
	)
	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER(TEXT("Init"))

		LinkData.Reset();
		EffectorTargets.Reset();
		EffectorLinkIndices.Reset();
		LinkDataToHierarchyIndices.Reset();
		HierarchyToLinkDataMap.Reset();

		// verify the chain
		const TArrayView<const FFBIKEndEffector> EffectorsView(Effectors.GetData(), Effectors.Num());
		AddEffectors(Hierarchy, Root, EffectorsView, LinkData, EffectorTargets, EffectorLinkIndices, LinkDataToHierarchyIndices, HierarchyToLinkDataMap, SolverProperty);
	}

	DECLARE_SCOPE_HIERARCHICAL_COUNTER(TEXT("Update"))

	if (LinkDataToHierarchyIndices.Num() > 0)
	{
		// we do this every frame for now
		if (Constraints.Num() > 0)
		{
			DECLARE_SCOPE_HIERARCHICAL_COUNTER(TEXT("Build Constraint"))
			//Build constraints
			const TArrayView<const FFBIKConstraintOption> ConstraintsView(Constraints.GetData(), Constraints.Num());
			FBIKConstraintLib::BuildConstraints(ConstraintsView, InternalConstraints, Hierarchy, LinkData, LinkDataToHierarchyIndices, HierarchyToLinkDataMap);
		}

		// during only editor and update
		// we expect solver type changes, it will reinit
		// InternalConstraints can't be changed during runtime
		if (InternalConstraints.Num() > 0)
		{
			WorkData.IKSolver.SetPostProcessDelegateForIteration(FPostProcessDelegateForIteration::CreateStatic(&FBIKConstraintLib::ApplyConstraint, &InternalConstraints));
		}
		else
		{
			WorkData.IKSolver.ClearPostProcessDelegateForIteration();
		}
		// first set mid effector's transform

		// before update we finalize motion scale
		// this code may go away once we have constraint
		// update link data and end effectors
		for (int32 LinkIndex = 0; LinkIndex < LinkData.Num(); ++LinkIndex)
		{
			const FRigElementKey& Item = *LinkDataToHierarchyIndices.Find(LinkIndex);
			LinkData[LinkIndex].SetTransform(Hierarchy->GetGlobalTransform(Item));
			// @todo: fix this somewhere else - we can add this to prepare step
			// @todo: we update motion scale here, then?
			LinkData[LinkIndex].FinalizeForSolver();
		}

		// update mid effector info

		const float LinearMotionStrength = FMath::Max(SolverProperty.LinearMotionStrength, SolverProperty.MinLinearMotionStrength);
		const float AngularMotionStrength = FMath::Max(SolverProperty.AngularMotionStrength, SolverProperty.MinAngularMotionStrength);
		const float LinearRange = LinearMotionStrength - SolverProperty.MinLinearMotionStrength;
		const float AngularRange = AngularMotionStrength - SolverProperty.MinAngularMotionStrength;

		// update end effector info
		for (int32 EffectorIndex = 0; EffectorIndex < Effectors.Num(); ++EffectorIndex)
		{
			int32 EffectorLinkIndex = EffectorLinkIndices[EffectorIndex];
			if (EffectorLinkIndex != INDEX_NONE)
			{
				FFBIKEffectorTarget* EffectorTarget = EffectorTargets.Find(EffectorLinkIndex);
				if (EffectorTarget)
				{
					const FFBIKEndEffector& CurEffector = Effectors[EffectorIndex];
					const FVector CurrentLinkLocation = LinkData[EffectorLinkIndex].GetTransform().GetLocation();
					const FQuat CurrentLinkRotation = LinkData[EffectorLinkIndex].GetTransform().GetRotation();
					const FVector& EffectorLocation = CurEffector.Position;
					const FQuat& EffectorRotation = CurEffector.Rotation;
					EffectorTarget->Position = FMath::Lerp(CurrentLinkLocation, EffectorLocation, CurEffector.PositionAlpha);
					EffectorTarget->Rotation = FMath::Lerp(CurrentLinkRotation, EffectorRotation, CurEffector.RotationAlpha);
					EffectorTarget->InitialPositionDistance = (EffectorLocation - CurrentLinkLocation).Size();
					EffectorTarget->InitialRotationDistance = (FBIKUtil::GetScaledRotationAxis(EffectorRotation) - FBIKUtil::GetScaledRotationAxis(CurrentLinkRotation)).Size();

					const float Pull = FMath::Clamp(CurEffector.Pull, 0.f, 1.f);
					// we want some impact of Pull, in order for Pull to have some impact, we clamp to some number
					const float TargetClamp = FMath::Clamp(SolverProperty.DefaultTargetClamp, 0.f, 0.7f);
					const float Scale = TargetClamp + Pull * (1.f - TargetClamp);
					// Pull set up
					EffectorTarget->LinearMotionStrength = LinearRange * Scale + SolverProperty.MinLinearMotionStrength;
					EffectorTarget->AngularMotionStrength = AngularRange * Scale + SolverProperty.MinAngularMotionStrength;
					EffectorTarget->ConvergeScale = Scale;
					EffectorTarget->TargetClampScale = Scale;

					EffectorTarget->bPositionEnabled = true;
					EffectorTarget->bRotationEnabled = true;
				}
			}
		}

		TArray<FJacobianDebugData>& DebugData = WorkData.DebugData;
		DebugData.Reset();

		const bool bDebugEnabled = DebugOption.bDrawDebugHierarchy || DebugOption.bDrawDebugEffector || DebugOption.bDrawDebugConstraints;

		// we can't reuse memory until we fix the memory issue on RigVM
		{
			DECLARE_SCOPE_HIERARCHICAL_COUNTER(TEXT("Solver"))
			FJacobianSolver_FullbodyIK& IKSolver = WorkData.IKSolver;
			IKSolver.SolveJacobianIK(LinkData, EffectorTargets,
				JacobianIK::FSolverParameter(SolverProperty.Damping, true, false, (SolverProperty.bUseJacobianTranspose) ? EJacobianSolver::JacobianTranspose : EJacobianSolver::JacobianPIDLS),
				SolverProperty.MaxIterations, SolverProperty.Precision, (bDebugEnabled) ? &DebugData : nullptr);

			if (MotionProperty.bForceEffectorRotationTarget)
			{
				// if position is reached, we force rotation target
				for (int32 EffectorIndex = 0; EffectorIndex < Effectors.Num(); ++EffectorIndex)
				{
					int32 EffectorLinkIndex = EffectorLinkIndices[EffectorIndex];
					if (EffectorLinkIndex != INDEX_NONE)
					{
						FFBIKEffectorTarget* EffectorTarget = EffectorTargets.Find(EffectorLinkIndex);
						if (EffectorTarget && EffectorTarget->bRotationEnabled)
						{
							bool bApplyRotation = true;

							if (MotionProperty.bOnlyApplyWhenReachedToTarget)
							{
								// only do this when position is reached? This will conflict with converge scale
								const FVector& BonePosition = LinkData[EffectorLinkIndex].GetTransform().GetLocation();
								const FVector& TargetPosition = EffectorTarget->Position;

								bApplyRotation = (FVector(BonePosition-TargetPosition).SizeSquared() <= SolverProperty.Precision * SolverProperty.Precision);
							}

							if (bApplyRotation)
							{
								FQuat NewRotation = EffectorTarget->Rotation;
								FTransform NewTransform = LinkData[EffectorLinkIndex].GetTransform();
								NewTransform.SetRotation(NewRotation);
								LinkData[EffectorLinkIndex].SetTransform(NewTransform);
							}
						}
					}
				}
			}
		}

		///////////////////////////////////////////////////////////////////////////
		// debug draw start
		///////////////////////////////////////////////////////////////////////////
		if (bDebugEnabled && ExecuteContext.GetDrawInterface() != nullptr)
		{
			const int32 DebugDataNum = DebugData.Num();
			if (DebugData.Num() > 0)
			{
				for (int32 DebugIndex = DebugDataNum - 1; DebugIndex >= 0; --DebugIndex)
				{
					const TArray<FFBIKLinkData>& LocalLink = DebugData[DebugIndex].LinkData;

					FTransform Offset = DebugOption.DrawWorldOffset;
					Offset.SetLocation(Offset.GetLocation() * (DebugDataNum - DebugIndex));

					if (DebugOption.bDrawDebugHierarchy)
					{
						for (int32 LinkIndex = 0; LinkIndex < LocalLink.Num(); ++LinkIndex)
						{
							const FFBIKLinkData& Data = LocalLink[LinkIndex];

							FLinearColor DrawColor = FLinearColor::White;

							float LineThickness = 0.f;
							if (DebugOption.bColorAngularMotionStrength || DebugOption.bColorLinearMotionStrength)
							{
								DrawColor = FLinearColor::Black;
								if (DebugOption.bColorAngularMotionStrength)
								{
									const float Range = FMath::Max(SolverProperty.AngularMotionStrength - SolverProperty.MinAngularMotionStrength, 0.f);
									if (Range > 0.f)
									{
										float CurrentStrength = Data.GetAngularMotionStrength() - SolverProperty.MinAngularMotionStrength;
										float Alpha = FMath::Clamp(CurrentStrength / Range, 0.f, 1.f);
										DrawColor.R = LineThickness = Alpha;
									}
								}
								else if (DebugOption.bColorLinearMotionStrength)
								{
									const float Range = FMath::Max(SolverProperty.LinearMotionStrength - SolverProperty.MinLinearMotionStrength, 0.f);
									if (Range > 0.f)
									{
										float CurrentStrength = Data.GetLinearMotionStrength() - SolverProperty.MinLinearMotionStrength;
										float Alpha = FMath::Clamp(CurrentStrength / Range, 0.f, 1.f);
										DrawColor.B = LineThickness = Alpha;
									}
								}
							}

							if (Data.ParentLinkIndex != INDEX_NONE)
							{
								const FFBIKLinkData& ParentData = LocalLink[Data.ParentLinkIndex];
								ExecuteContext.GetDrawInterface()->DrawLine(Offset, Data.GetPreviousTransform().GetLocation(), ParentData.GetPreviousTransform().GetLocation(), DrawColor, LineThickness);
							}

							if (DebugOption.bDrawDebugAxes)
							{
								ExecuteContext.GetDrawInterface()->DrawAxes(Offset, Data.GetPreviousTransform(), DebugOption.DrawSize);
							}
						}
					}

					if (DebugOption.bDrawDebugEffector)
					{
						for (auto Iter = EffectorTargets.CreateConstIterator(); Iter; ++Iter)
						{
							const FFBIKEffectorTarget& EffectorTarget = Iter.Value();
							if (EffectorTarget.bPositionEnabled)
							{
								// draw effector target locations
								ExecuteContext.GetDrawInterface()->DrawBox(Offset, FTransform(EffectorTarget.Position), FLinearColor::Yellow, DebugOption.DrawSize);
							}

							// draw effector link location
							ExecuteContext.GetDrawInterface()->DrawBox(Offset, LocalLink[Iter.Key()].GetPreviousTransform(), FLinearColor::Green, DebugOption.DrawSize);
						}

						for (int32 Index = 0; Index < DebugData[DebugIndex].TargetVectorSources.Num(); ++Index)
						{
							// draw arrow to the target
							ExecuteContext.GetDrawInterface()->DrawLine(Offset, DebugData[DebugIndex].TargetVectorSources[Index].GetLocation(), 
								DebugData[DebugIndex].TargetVectorSources[Index].GetLocation() + DebugData[DebugIndex].TargetVectors[Index], FLinearColor::Red);
						}
					}
				}
			}

			if (DebugOption.bDrawDebugConstraints && InternalConstraints.Num())
			{
				FTransform Offset = FTransform::Identity;

				// draw frame if active
				for (int32 Index = 0; Index < Constraints.Num(); ++Index)
				{
					if (Constraints[Index].bEnabled)
					{
						if (Constraints[Index].Item.IsValid())
						{
							const int32* Found = HierarchyToLinkDataMap.Find(Constraints[Index].Item);
							if (Found)
							{
								FTransform ConstraintFrame = LinkData[*Found].GetTransform();
								ConstraintFrame.ConcatenateRotation(FQuat(Constraints[Index].OffsetRotation));
								ExecuteContext.GetDrawInterface()->DrawAxes(Offset, ConstraintFrame, 2.f);
							}
						}
					}
				}

				for (int32 Index = 0; Index < InternalConstraints.Num(); ++Index)
				{
					// for now we have rotation limit only
					if (InternalConstraints[Index].IsType< FRotationLimitConstraint>())
					{
						FRotationLimitConstraint& LimitConstraint = InternalConstraints[Index].Get<FRotationLimitConstraint>();
						const FQuat LocalRefRotation = LimitConstraint.RelativelRefPose.GetRotation();
						FTransform RotationTransform = LinkData[LimitConstraint.ConstrainedIndex].GetTransform();
						// base is parent transform but in their space, we can get there by inversing local ref rotation
						FTransform BaseTransform = FTransform(LocalRefRotation).GetRelativeTransformReverse(RotationTransform);
						BaseTransform.ConcatenateRotation(LimitConstraint.BaseFrameOffset);
						ExecuteContext.GetDrawInterface()->DrawAxes(Offset, BaseTransform, 5.f, 1.f);

						// current transform
						const FQuat LocalRotation = BaseTransform.GetRotation().Inverse() * RotationTransform.GetRotation();
						const FQuat DeltaTransform = LocalRefRotation.Inverse() * LocalRotation;
						RotationTransform.SetRotation(BaseTransform.GetRotation() * DeltaTransform);
						RotationTransform.NormalizeRotation();
						RotationTransform.SetLocation(BaseTransform.GetLocation());

						// draw ref pose on their current transform
						ExecuteContext.GetDrawInterface()->DrawAxes(Offset, RotationTransform, 10.f, 1.f);

						FVector XAxis = BaseTransform.GetUnitAxis(EAxis::X);
						FVector YAxis = BaseTransform.GetUnitAxis(EAxis::Y);
						FVector ZAxis = BaseTransform.GetUnitAxis(EAxis::Z);

						if (LimitConstraint.bXLimitSet)
						{
							FTransform XAxisConeTM(YAxis, XAxis ^ YAxis, XAxis, BaseTransform.GetTranslation());
							XAxisConeTM.SetRotation(FQuat(XAxis, 0.f) * XAxisConeTM.GetRotation());
							XAxisConeTM.SetScale3D(FVector(30.f));
							ExecuteContext.GetDrawInterface()->DrawCone(Offset, XAxisConeTM, LimitConstraint.Limit.X, 0.0f, 24, false, FLinearColor::Red, GEngine->ConstraintLimitMaterialX->GetRenderProxy());
						}

						if (LimitConstraint.bYLimitSet)
						{
							FTransform YAxisConeTM(ZAxis, YAxis ^ ZAxis, YAxis, BaseTransform.GetTranslation());
							YAxisConeTM.SetRotation(FQuat(YAxis, 0.f) * YAxisConeTM.GetRotation());
							YAxisConeTM.SetScale3D(FVector(30.f));
							ExecuteContext.GetDrawInterface()->DrawCone(Offset, YAxisConeTM, LimitConstraint.Limit.Y, 0.0f, 24, false, FLinearColor::Green, GEngine->ConstraintLimitMaterialY->GetRenderProxy());
						}

						if (LimitConstraint.bZLimitSet)
						{
							FTransform ZAxisConeTM(XAxis, ZAxis ^ XAxis, ZAxis, BaseTransform.GetTranslation());
							ZAxisConeTM.SetRotation(FQuat(ZAxis, 0.f) * ZAxisConeTM.GetRotation());
							ZAxisConeTM.SetScale3D(FVector(30.f));
							ExecuteContext.GetDrawInterface()->DrawCone(Offset, ZAxisConeTM, LimitConstraint.Limit.Z, 0.0f, 24, false, FLinearColor::Blue, GEngine->ConstraintLimitMaterialZ->GetRenderProxy());
						}
					}

					if (InternalConstraints[Index].IsType<FPoleVectorConstraint>())
					{
						// darw pole vector location
						// draw 3 joints line and a plane to pole vector
						FPoleVectorConstraint& Constraint = InternalConstraints[Index].Get<FPoleVectorConstraint>();
						FTransform RootTransform = LinkData[Constraint.ParentBoneIndex].GetTransform();
						FTransform JointTransform = LinkData[Constraint.BoneIndex].GetTransform();
						FTransform ChildTransform = LinkData[Constraint.ChildBoneIndex].GetTransform();

						FVector JointTarget = (Constraint.bUseLocalDir)? Constraint.CalculateCurrentPoleVectorDir(RootTransform, JointTransform, ChildTransform, LinkData[Constraint.BoneIndex].LocalFrame) : Constraint.PoleVector;

						// draw the plane, 
						TArray<FVector> Positions;
						Positions.Add(RootTransform.GetLocation());
						Positions.Add(ChildTransform.GetLocation());
						Positions.Add(ChildTransform.GetLocation());
						Positions.Add(JointTarget);
						Positions.Add(JointTarget);
						Positions.Add(RootTransform.GetLocation());

						ExecuteContext.GetDrawInterface()->DrawLines(Offset, Positions, FLinearColor::Gray, 1.2f);
						ExecuteContext.GetDrawInterface()->DrawLine(Offset, JointTransform.GetLocation(), JointTarget, FLinearColor::Red, 1.2f);
					}
				}
			}
		}
		///////////////////////////////////////////////////////////////////////////
		// debug draw end
		///////////////////////////////////////////////////////////////////////////
	}

	// we update back to hierarchy
	for (int32 LinkIndex= 0; LinkIndex < LinkData.Num() ; ++LinkIndex)
	{
		// only propagate, if you are leaf joints here
		// this means, only the last joint in the test
		const FRigElementKey& CurrentItem = *LinkDataToHierarchyIndices.Find(LinkIndex);
		const FTransform& LinkTransform = LinkData[LinkIndex].GetTransform();
		Hierarchy->SetGlobalTransform(CurrentItem, LinkTransform, false, bPropagateToChildren);
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Stats/StatsHierarchical.h"
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_FullbodyIK)
{
/*	BoneHierarchy.Add(TEXT("Root"), NAME_None, ERigBoneType::User, FTransform(FVector(1.f, 0.f, 0.f)));
	// add  2 chains of length 2,2 
	BoneHierarchy.Add(TEXT("Chain1_0"), TEXT("Root"), ERigBoneType::User, FTransform(FVector(1.f, 2.f, 0.f)));
	BoneHierarchy.Add(TEXT("Chain1_1"), TEXT("Chain1_0"), ERigBoneType::User, FTransform(FVector(3.f, 2.f, 0.f)));
	//length 3,3
	BoneHierarchy.Add(TEXT("Chain2_0"), TEXT("Root"), ERigBoneType::User, FTransform(FVector(-2.f, 0.f, 0.f)));
	BoneHierarchy.Add(TEXT("Chain2_1"), TEXT("Chain2_0"), ERigBoneType::User, FTransform(FVector(-2.f, 3.f, 0.f)));

	BoneHierarchy.Initialize();
	Unit.ExecuteContext.Hierarchy = &HierarchyContainer;

	// first validation test
	// make sure this doesn't crash
	InitAndExecute();

	Unit.RootBone = TEXT("Root");
	Unit.Effectors.AddDefaulted(2);

	// second make sure this doesn't crash
	InitAndExecute();

	// now add the data
	Unit.Effectors[0].Bone = TEXT("Chain1_1");
	Unit.Effectors[0].Position = FVector(3.f, 2.f, 0.f);
	Unit.Effectors[1].Bone = TEXT("Chain2_1");
	Unit.Effectors[1].Position = FVector(-2.f, 3.f, 0.f);
	Unit.bPropagateToChildren = true;

	InitAndExecute();
	AddErrorIfFalse(BoneHierarchy.GetGlobalTransform(TEXT("Chain1_1")).GetTranslation().Equals(FVector(3.f, 2.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(BoneHierarchy.GetGlobalTransform(TEXT("Chain2_1")).GetTranslation().Equals(FVector(-2.f, 3.f, 0.f)), TEXT("unexpected transform"));

	// root is (1, 0, 0)
	Unit.Effectors[0].Bone = TEXT("Chain1_1");
	Unit.Effectors[0].Position = FVector(4.f, 0.f, 0.f);
	Unit.Effectors[1].Bone = TEXT("Chain2_1");
	Unit.Effectors[1].Position = FVector(0.f, -5.f, 0.f);
	Unit.bPropagateToChildren = true;

	InitAndExecute();
	AddErrorIfFalse(BoneHierarchy.GetGlobalTransform(TEXT("Chain1_1")).GetTranslation().Equals(FVector(4.f, 0.f, 0.f), 0.1f), TEXT("unexpected transform"));
	AddErrorIfFalse(BoneHierarchy.GetGlobalTransform(TEXT("Chain2_1")).GetTranslation().Equals(FVector(0.f, -5.f, 0.f), 0.1f), TEXT("unexpected transform"));*/
	return true;
}
#endif
FRigVMStructUpgradeInfo FRigUnit_FullbodyIK::GetUpgradeInfo() const
{
	return FRigUnit_HighlevelBaseMutable::GetUpgradeInfo();
}

