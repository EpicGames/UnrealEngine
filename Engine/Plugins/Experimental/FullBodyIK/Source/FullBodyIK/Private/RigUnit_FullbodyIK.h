// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FBIKConstraint.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "FBIKConstraintOption.h"
#include "FBIKDebugOption.h"
#include "FBIKShared.h"
#include "Misc/TVariantMeta.h"
#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigUnit_FullbodyIK.generated.h"

struct FControlRigExecuteContext;
struct FRigVMStructUpgradeInfo;

USTRUCT()
struct FFBIKEndEffector
{
	GENERATED_BODY()
	/**
	 * The last item in the chain to solve - the effector
	 */
	UPROPERTY(meta = (Constant, CustomWidget = "BoneName"))
	FRigElementKey Item;

	UPROPERTY()
	FVector Position;

	UPROPERTY()
	float	PositionAlpha = 1.f;

	UPROPERTY()
	int32  PositionDepth = 1000;
	
	UPROPERTY()
	FQuat	Rotation;

	UPROPERTY()
	float	RotationAlpha = 1.f;

	UPROPERTY()
	int32  RotationDepth = 1000;

	/*
	 * Clamps the total length to target by this scale for each iteration 
	 * This helps to stabilize solver to reduce singularity by avoiding to try to reach target too far. 
	 */
	UPROPERTY()
	float	Pull = 0.f;

	FFBIKEndEffector()
		: Item(NAME_None, ERigElementType::Bone)
		, Position(FVector::ZeroVector)
		, Rotation(FQuat::Identity)
	{
	}

	FFBIKEndEffector(const FFBIKEndEffector& Other)
	{
		*this = Other;
	}

	FFBIKEndEffector& operator = (const FFBIKEndEffector& Other)
	{
		Item = Other.Item;
		Position = Other.Position;
		PositionAlpha = Other.PositionAlpha;
		PositionDepth = Other.PositionDepth;
		Rotation = Other.Rotation;
		RotationAlpha = Other.RotationAlpha;
		RotationDepth = Other.RotationDepth;
		Pull = Other.Pull;
		return *this;
	}
};

USTRUCT()
struct FRigUnit_FullbodyIK_WorkData
{
	GENERATED_BODY()

	FRigUnit_FullbodyIK_WorkData()
	{
	}

	/** list of Link Data for solvers - joints */
	TArray<FFBIKLinkData> LinkData;
	/** Effector Targets - search key is LinkData Index */
	TMap<int32, FFBIKEffectorTarget> EffectorTargets; 
	/** End Effector Link Indices - EndEffector index to LinkData index*/
	TArray<int32> EffectorLinkIndices;
	/** Map from LinkData index to Rig Hierarchy Index*/
	TMap<int32, FRigElementKey> LinkDataToHierarchyIndices;
	/** Map from Rig Hierarchy Index to LinkData index*/
	TMap<FRigElementKey, int32> HierarchyToLinkDataMap;
	/** Constraints data */
	TArray<ConstraintType> InternalConstraints;
	/* Current Solver */
	FJacobianSolver_FullbodyIK IKSolver;
	/** Debug Data */
	TArray<FJacobianDebugData> DebugData;
};

/**
 * Based on Jacobian solver at core, this can solve multi chains within a root using multi effectors
 */
USTRUCT(meta=(DisplayName="Fullbody IK", Category="Hierarchy", Keywords="Multi, Effector, N-Chain, FB, IK", Deprecated = "5.0"))
struct FRigUnit_FullbodyIK : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	FRigUnit_FullbodyIK()
		: Root(NAME_None, ERigElementType::Bone)
	{
		MotionProperty.bForceEffectorRotationTarget = true;
	}

	/**
	 * The first bone in the chain to solve
	 */
	UPROPERTY(meta = (Input, Constant, CustomWidget = "BoneName"))
	FRigElementKey Root;

	UPROPERTY(meta = (Input))
	TArray<FFBIKEndEffector> Effectors;

	UPROPERTY(EditAnywhere, Category = FRigUnit_Jacobian, meta = (Input))
	TArray<FFBIKConstraintOption> Constraints;

	UPROPERTY(meta = (Input, Constant))
	FSolverInput	SolverProperty;

	UPROPERTY(meta = (Input, Constant))
	FMotionProcessInput MotionProperty;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input))
	bool bPropagateToChildren = true;

	UPROPERTY(meta = (Input))
	FFBIKDebugOption DebugOption;

	UPROPERTY(transient)
	FRigUnit_FullbodyIK_WorkData WorkData;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};
