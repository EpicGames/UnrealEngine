// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "Misc/TVariant.h"

#define UE_API FULLBODYIK_API

enum class EConstraintType : uint8
{
	Free,
	Limit, // currently hard limit
	Locked,
};

struct FMotionBase
{
	FMotionBase(const FVector& InBaseAxis)
		: BaseAxis(InBaseAxis)
		, LinearStiffness(1.f)
		, AngularStiffness(0.f)
	{}

	void SetAngularStiffness(float InStiffness)
	{
		AngularStiffness = FMath::Clamp(InStiffness, 0.f, 1.f);
	}

	void SetLinearStiffness(float InStiffness)
	{
		LinearStiffness = FMath::Clamp(InStiffness, 0.f, 1.f);
	}

	float GetAngularMotionScale() const
	{
		return (1.f - AngularStiffness);
	}

	float GetLinearMotionScale() const
	{
		return (1.f - LinearStiffness);
	}

	float GetAngularStiffness() const
	{
		return AngularStiffness;
	}

	float GetLinearStiffness() const
	{
		return LinearStiffness;
	}

	bool IsLinearMotionAllowed() const
	{
		// if stiff, then it can't move
		return (LinearStiffness < 1.f);
	}

	bool IsAngularMotionAllowed() const
	{
		// if stiff, then it can't move
		return (AngularStiffness < 1.f);
	}

	// rotation axis that this motion represents
	FVector	BaseAxis;

private:

	// this is in range of [0, 1] to define how stiff this is for this rotaiton axis
	// if 1, it doesn't move
	float	LinearStiffness;
	float	AngularStiffness;
};

struct FFBIKLinkData
{
	FFBIKLinkData()
		: ParentLinkIndex(INDEX_NONE)
		, Length(0.f)
		, LinkTransform(FTransform::Identity)
		, LinearMotionStrength(-1.f)
		, AngularMotionStrength(-1.f)
		, LocalFrame(FQuat::Identity)
	{
	}

	~FFBIKLinkData() = default;

	int32 GetNumMotionBases() const { return MotionBaseAxes.Num(); }

	// return true if any of my motion bases allow linear motion
	bool IsLinearMotionAllowed() const
	{
		// if any of axis allow, we allow
		for (const FMotionBase& Axis : MotionBaseAxes)
		{
			if (Axis.IsLinearMotionAllowed())
			{
				return true;
			}
		}

		return false;
	}

	// return true if any specific motion base by index allows linear motion
	bool IsLinearMotionAllowed(int32 BaseIndex) const
	{
		if (MotionBaseAxes.IsValidIndex(BaseIndex))
		{
			return MotionBaseAxes[BaseIndex].IsLinearMotionAllowed();
		}

		return false;
	}

	// return true if any of my motion bases allow angular motion
	bool IsAngularMotionAllowed() const
	{
		// if any of axis allow, we allow
		for (const FMotionBase& Axis : MotionBaseAxes)
		{
			if (Axis.IsAngularMotionAllowed())
			{
				return true;
			}
		}

		return false;
	}

	// return true if any specific motion base by index allows linear motion
	bool IsAngularMotionAllowed(int32 BaseIndex) const
	{
		if (MotionBaseAxes.IsValidIndex(BaseIndex))
		{
			return MotionBaseAxes[BaseIndex].IsAngularMotionAllowed();
		}

		return false;
	}

	// this is with 
	void AddMotionStrength(float InLinearScale, float InAngularScale)
	{
		// currently we only use Max, so we may not need to save this
		// for now it's easy to debug or tweak if we'd like to change it
		// so save to array
		LinearMotionStrengthArray.Add(InLinearScale);
		AngularMotionStrengthArray.Add(InAngularScale);
	}

	// @todo: may refactor this to somewhere else
	// currently I don't really have to save this anymore 
	void FinalizeForSolver()
	{
		// currently we only do Max
		// but it would be interesting to play with how this strength is calculated from effector information
		if (LinearMotionStrengthArray.Num() > 0)
		{
			LinearMotionStrength = 0.f;
			for (float Scale : LinearMotionStrengthArray)
			{
				LinearMotionStrength = FMath::Max(Scale, LinearMotionStrength);
			}

			AngularMotionStrength = 0.f;
			for (float Scale : AngularMotionStrengthArray)
			{
				AngularMotionStrength = FMath::Max(Scale, AngularMotionStrength);
			}
		}
		else
		{
			LinearMotionStrength = 0.f;
			AngularMotionStrength = 0.f;
		}
	}

	//@TODO:  stiff ness per rotation axis?
	float  GetLinearMotionStrength() const
	{
		return LinearMotionStrength;
	}

	float GetAngularMotionStrength() const
	{
		return AngularMotionStrength;
	}

	void ResetMotionBases()
	{
		MotionBaseAxes.Reset();
	}

	void AddMotionBase(const FMotionBase& InLinkAxis)
	{
		if (!InLinkAxis.BaseAxis.Equals(FVector(0.f)))
		{
			MotionBaseAxes.Add(InLinkAxis);
		}
	}

	const FMotionBase& GetMotionBase(int32 Index) const
	{
		check(MotionBaseAxes.IsValidIndex(Index));
		return MotionBaseAxes[Index];
	}

	FMotionBase& GetMotionBase(int32 Index)
	{
		check(MotionBaseAxes.IsValidIndex(Index));
		return MotionBaseAxes[Index];
	}

	const FVector& GetRotationAxis(int32 Index) const
	{
		if (MotionBaseAxes.IsValidIndex(Index))
		{
			return MotionBaseAxes[Index].BaseAxis;
		}

		ensure(false);
		return FVector::ForwardVector;
	}


	int32	ParentLinkIndex;
	float	Length; // reference link length

	void SetTransform(const FTransform& InTransform)
	{
		LinkTransform = InTransform;
		LinkTransform.NormalizeRotation();
	}

	const FTransform& GetTransform() const
	{
		return LinkTransform;
	}

	void SavePreviousTransform()
	{
		PreviousLinkTransform = LinkTransform;
	}

	const FTransform& GetPreviousTransform() const
	{
		return PreviousLinkTransform;
	}

private:

	FTransform LinkTransform; // link position is based on root space
	FTransform PreviousLinkTransform; // before modified, we save this for debugging

	// this motion strength is calculated by effector's strength
	// but stiffness can control that range, where this joint lies
	// stiffness is expected to be in range of [0, 1]
	float	LinearMotionStrength;
	float	AngularMotionStrength;

	TArray<float> LinearMotionStrengthArray;
	TArray<float> AngularMotionStrengthArray;

	// this is the number of axes, it wants to generate motion w.r.t.
	// this is often rotation axis in the jacobian method, 
	// but it does contain more property on how it applies it
	// Stiff is now moved Motion Base. Each axis allow different motion property
	TArray<FMotionBase> MotionBaseAxes;

public:
	// local frame - ideally we can make this to be angular vs linear different, but
	// for now we just have one frame
	FQuat LocalFrame;
};

struct FPositionLimitConstraint 
{
	FPositionLimitConstraint()
		: BaseIndex(INDEX_NONE)
		, ConstrainedIndex(INDEX_NONE)
		, bXLimitSet(false)
		, bYLimitSet(false)
		, bZLimitSet(false)
		, Limit(0.f)
	{}

	int32 BaseIndex;
	int32 ConstrainedIndex;

	// frame of reference for this
	FTransform	RelativelRefPose;

	// xyz
	uint8 bXLimitSet : 1;
	uint8 bYLimitSet : 1;
	uint8 bZLimitSet : 1;

	FVector Limit;

	bool ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<FTransform>& InOutLocalTransforms);
};

struct FRotationLimitConstraint
{
	FRotationLimitConstraint()
		: BaseIndex(INDEX_NONE)
		, ConstrainedIndex(INDEX_NONE)
		, bXLimitSet(false)
		, bYLimitSet(false)
		, bZLimitSet(false)
		, Limit(0.f)
	{}

	int32 BaseIndex;
	int32 ConstrainedIndex;

	// frame of reference for this
	FQuat		BaseFrameOffset; // include any offset and in their local space
	FTransform	RelativelRefPose;

	// xyz - twist/swing1/swing2
	uint8 bXLimitSet : 1;
	uint8 bYLimitSet : 1;
	uint8 bZLimitSet : 1;

	FVector Limit;

	bool ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<FTransform>& InOutLocalTransforms);
};

struct FPoleVectorConstraint 
{
	int32 BoneIndex;
	int32 ParentBoneIndex;
	int32 ChildBoneIndex;

	// if you use local dir, it will be local dir and will be calculated based on local frame every time
	// if not, it will interpret PoleVector as a target location in the space (global space)
	bool	bUseLocalDir = true;
	FVector PoleVector;

	UE_API bool ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<FTransform>& InOutLocalTransforms);
	UE_API FVector CalculateCurrentPoleVectorDir(const FTransform& ParentTransform, const FTransform& BoneTransform, const FTransform& ChildTransform, const FQuat& LocalFrame) const;
};


using ConstraintType = TVariant<FRotationLimitConstraint, FPositionLimitConstraint, FPoleVectorConstraint>;

#undef UE_API
