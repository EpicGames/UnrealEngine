// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FBIKConstraint.h"

struct FFBIKConstraintOption;
struct FRigElementKey;
class URigHierarchy;

namespace FBIKConstraintLib
{
	// constraint related
	void FULLBODYIK_API ApplyConstraint(TArray<FFBIKLinkData>& InOutLinkData, TArray<ConstraintType>* Constraints);
	void FULLBODYIK_API BuildConstraints(const TArrayView<const FFBIKConstraintOption>& Constraints, TArray<ConstraintType>& OutConstraints,
		const URigHierarchy* Hierarchy, TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FRigElementKey>& LinkDataToHierarchyIndices,
		const TMap<FRigElementKey, int32>& HierarchyToLinkDataMap);
};
