// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyDefines.h"
#include "FBIKConstraintOption.generated.h"

UENUM()
enum class EFBIKBoneLimitType : uint8
{
	Free,
	Limit, // currently hard limit
	Locked,
};

USTRUCT()
struct FFBIKBoneLimit
{
	GENERATED_BODY()

	FFBIKBoneLimit()
		: LimitType_X(EFBIKBoneLimitType::Locked)
		, LimitType_Y(EFBIKBoneLimitType::Locked)
		, LimitType_Z(EFBIKBoneLimitType::Locked)
		, Limit(EForceInit::ForceInitToZero)
	{}

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	EFBIKBoneLimitType LimitType_X;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	EFBIKBoneLimitType LimitType_Y;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	EFBIKBoneLimitType LimitType_Z;

	// currently no negative limit, for position we may need min/max
	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	FVector Limit; // in their local space
};

UENUM()
enum class EPoleVectorOption : uint8
{
	Direction, /* The polevector will indicate a direction vector in their local bone space */
	Location,   /* The polevector will indicate a location in their global space(or component space) */
};

USTRUCT()
struct FFBIKConstraintOption
{
	GENERATED_BODY()

	FFBIKConstraintOption()
		: Item(NAME_None, ERigElementType::Bone)
		, LinearStiffness(FVector(1.f))
		, AngularStiffness(FVector::ZeroVector)
		, PoleVector(FVector::RightVector)
		, OffsetRotation(FRotator::ZeroRotator)
	{
		AngularLimit.LimitType_X = EFBIKBoneLimitType::Free;
		AngularLimit.LimitType_Y = EFBIKBoneLimitType::Free;
		AngularLimit.LimitType_Z = EFBIKBoneLimitType::Free;
		
	}

	/** Bone Name */
	UPROPERTY(EditAnywhere, meta = (Constant), Category=FFBIKConstraintOption)
	FRigElementKey Item;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	bool bEnabled = true;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	bool bUseStiffness = true;

	/** Scale of [0, 1] and applied to linear motion strength - linear stiffness works on their local frame*/
	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	FVector LinearStiffness; 

	/** Scale of [0, 1] and applied to angular motion strength, xyz is applied to twist, swing1, swing2 */
	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	FVector AngularStiffness;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	bool bUseAngularLimit = false;

	/** Angular delta limit of this joints local transform. [-Limit, Limit] */
	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	FFBIKBoneLimit AngularLimit;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	bool bUsePoleVector = false;

	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	EPoleVectorOption	PoleVectorOption = EPoleVectorOption::Direction;

	/** Pole Vector in their local space*/
	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	FVector PoleVector; 

	// this is offset rotation applied when constructing the local frame
	UPROPERTY(EditAnywhere, Category = FFBIKConstraintOption)
	FRotator OffsetRotation;

	bool IsLinearlyLimited() const 
	{
		return false; // !(LinearLimit.LimitType_X == EBoneLimitType::Free && LinearLimit.LimitType_Y == EBoneLimitType::Free && LinearLimit.LimitType_Z == EBoneLimitType::Free);
	}

	bool IsAngularlyLimited() const
	{
		if (bUseAngularLimit)
		{
			return !(AngularLimit.LimitType_X == EFBIKBoneLimitType::Free && AngularLimit.LimitType_Y == EFBIKBoneLimitType::Free && AngularLimit.LimitType_Z == EFBIKBoneLimitType::Free);
		}

		return false;
	}
};

USTRUCT(BlueprintType)
struct FMotionProcessInput
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = FMotionProcessInput)
	bool	bForceEffectorRotationTarget = false;

	UPROPERTY(EditAnywhere, Category = FMotionProcessInput)
	bool	bOnlyApplyWhenReachedToTarget = false;
};
