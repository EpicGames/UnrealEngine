// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FBIKDebugOption.generated.h"

USTRUCT(BlueprintType)
struct FFBIKDebugOption
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	bool bDrawDebugHierarchy = false;

	// use red channel
	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	bool bColorAngularMotionStrength = false;

	// use green channel
	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	bool bColorLinearMotionStrength = false;

	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	bool bDrawDebugAxes = false;

	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	bool bDrawDebugEffector = false;

	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	bool bDrawDebugConstraints = false;

	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	FTransform DrawWorldOffset;

	UPROPERTY(EditAnywhere, Category = FFBIKDebugOption)
	float DrawSize = 5.f;

	FFBIKDebugOption()
	{
		DrawWorldOffset.SetLocation(FVector(30.f, 0.f, 0.f));
	}
};
