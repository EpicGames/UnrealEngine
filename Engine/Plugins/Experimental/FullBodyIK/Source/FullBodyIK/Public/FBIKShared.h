// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JacobianSolver.h"
#include "FBIKShared.generated.h"

#define UE_API FULLBODYIK_API

namespace JacobianIK { struct FFBIKEffectorTarget; }

class FJacobianSolver_FullbodyIK : public FJacobianSolverBase
{
public:
	FJacobianSolver_FullbodyIK() : FJacobianSolverBase() {}

private:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
	UE_API virtual void PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const override;
};

USTRUCT(BlueprintType)
struct FSolverInput
{
	GENERATED_BODY()

	/*
	 * This value is applied to the target information for effectors, which influence back to 
	 * Joint's motion that are affected by the end effector
	 * The reason min/max is used when we apply the depth through the chain that are affected

	 */
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float	LinearMotionStrength = 3.f;

	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float	MinLinearMotionStrength = 2.f;

	/*
	 * This value is applied to the target information for effectors, which influence back to 
	 * Joint's motion that are affected by the end effector
	 * The reason min/max is used when we apply the depth through the chain that are affected
	 */
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float	AngularMotionStrength = 3.f;

	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float	MinAngularMotionStrength = 2.f;

	/* This is a scale value (range from 0-0.7) that is used to stablize the target vector. If less, it's more stable, but it can reduce speed of converge. */
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float	DefaultTargetClamp = 0.2f;

	/**
	 * The precision to use for the solver
	 */
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float Precision = 0.1f;

	/**
	* The precision to use for the fabrik solver
	*/
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	float Damping = 30.f;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	int32 MaxIterations = 30;

	/**
	 * Cheaper solution than default Jacobian Pseudo Inverse Damped Least Square
	 */
	UPROPERTY(EditAnywhere, Category = FSolverInput)
	bool bUseJacobianTranspose = false;
};

#undef UE_API
