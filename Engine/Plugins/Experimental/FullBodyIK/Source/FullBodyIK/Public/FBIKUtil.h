// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"

class URigHierarchy;
struct FRigElementKey;

namespace FBIKUtil
{
	/** Utility functions */
	FVector FULLBODYIK_API GetScaledRotationAxis(const FQuat& InQuat);
	bool FULLBODYIK_API CanCrossProduct(const FVector& Vector1, const FVector& Vector2);
	bool FULLBODYIK_API GetBoneChain(URigHierarchy* Hierarchy, const FRigElementKey& Root, const FRigElementKey& Current, TArray<FRigElementKey>& ChainIndices);
};
