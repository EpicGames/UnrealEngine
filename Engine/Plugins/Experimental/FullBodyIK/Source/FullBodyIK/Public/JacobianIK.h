// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"

// Just to be sure, also added this in Eigen.Build.cs
#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

struct FFBIKLinkData;

namespace JacobianIK
{
	struct FFBIKEffectorTarget
	{
		/** Position Target **/
		bool	bPositionEnabled;
		FVector Position;

		/** Rotation Target **/
		bool	bRotationEnabled;
		FQuat	Rotation;

		/*
		 * This strength is applied to how strong this effector target is
		 * If you want the position to be more effective, you can increase this to be more precise and lower if you want less. 
		 */
		float LinearMotionStrength;
		float AngularMotionStrength;

		/*
		 * This is applied when testing converge to see if the effector is reached to target or not
		 * This allows to look of pull or reach based on this value
		 */
		float ConvergeScale;

		/* Total chain length of this effector from root. Used when clamping*/
		float ChainLength;

		/** ClampScale to target. Helps to stablize the target */
		float TargetClampScale;

		/* This is to support UpdateClamp for each iteration
		 * we recalculate the adjusted value and apply */
		float AdjustedLength;

		/** Initial distance prior to solver iteration from effetor to target */
		float InitialPositionDistance;

		/** Initial distance prior to solver iteration from effetor to target - currently not used to avoid having another precision value for input that is difficult to estimate*/
		float InitialRotationDistance;

		/** Save list of chain it affects */
		TArray<int32> LinkChain;

		FFBIKEffectorTarget()
			: Position(EForceInit::ForceInitToZero)
			, Rotation(EForceInit::ForceInitToZero)
			, LinearMotionStrength(0.5f)
			, AngularMotionStrength(0.5f)
			, ConvergeScale(0.5f)
			, ChainLength(0.f)
			, TargetClampScale(1.f)
			, AdjustedLength(0.f)
			, InitialPositionDistance(0.f)
			, InitialRotationDistance(0.f)
		{}

		float GetCurrentLength() const 
		{
			return ChainLength* TargetClampScale + AdjustedLength;
		}
	};

	enum class ERotationAxis : uint8
	{
		Twist,
		Swing1,
		Swing2,
	};

	enum class EJacobianSolver : uint8
	{
		JacobianTranspose,
		JacobianPIDLS,
	};

	struct FSolverParameter
	{
		/** Damping allows solver to reduce oscillation but it increase iteration time to converge */
		float DampingValue;

		/** Which solver to use */
		EJacobianSolver JacobianSolver;

		/** Enable clamping scale to target */
		bool bClampToTarget : 1;
		/** update clamping length for each iteration based on how much it moved from previous iteration */
		bool bUpdateClampMagnitude : 1;

		FSolverParameter()
			: DampingValue(10.f)
			, JacobianSolver(EJacobianSolver::JacobianPIDLS)
			, bClampToTarget(true)
			, bUpdateClampMagnitude(true)
		{}

		FSolverParameter(float InDampingValue, 
			bool bInClampToTarget, 
			bool bInUpdateClampMagnitude,
			EJacobianSolver InJacobianSolver = EJacobianSolver::JacobianPIDLS)
			: DampingValue(InDampingValue)
			, JacobianSolver(InJacobianSolver)
			, bClampToTarget(bInClampToTarget)
			, bUpdateClampMagnitude(bInUpdateClampMagnitude)
		{}
	};


	/** Delegates for core library to use to calculate partial derivatives */
	DECLARE_DELEGATE_RetVal_SixParams(FVector, FCalculatePartialDerivativesDelegate, 
		const FFBIKLinkData&/* InLinkData*/, 
		bool /*bPositionChange*/,
		int32 /*LinkComponentIndex*/,
		const FFBIKLinkData& /*InEffectorLinkData*/, 
		bool /*bPositionTarget*/,
		const FSolverParameter& /*SolverParam*/);

	/** Delegates for core library to use to calculate target vector for end effectors */
	DECLARE_DELEGATE_RetVal_FourParams(FVector, FCalculateTargetVectorDelegate, 
		const FFBIKLinkData& /*InEffectorLink*/, 
		const FFBIKEffectorTarget& /*InEffectorData*/, 
		bool /*bPositionTarget*/,
		const FSolverParameter& /*SolverParam*/);

	void AllocateMatrix(Eigen::MatrixXf& InOutMatrix,
		const TArray<FFBIKLinkData>& InLinkData,
		int32 NumLinkComponent/* per each link - each component is 1 elements*/,
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		int32 NumEffectorComponent/* per each effector - for example position or rotation - each component is 3 elements*/);

	FVector ComputePositionalPartialDerivative(
		const FFBIKLinkData& InLinkData, 
		const FFBIKLinkData& InEffectorLinkData, 
		const FVector& RotationAxis);

	/* Core jacobian solver functions */
	bool CreateJacobianMatrix(const TArray<FFBIKLinkData>& InLinkData, 
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		Eigen::MatrixXf& InOutMatrix, 
		FCalculatePartialDerivativesDelegate OnCalculatePartialDerivatives, 
		const FSolverParameter& SolverParam);

	/* Create Angle PArtial Derivatives using Jacobian Psuedo Inverse Damped Least Square */
	bool CreateAnglePartialDerivativesUsingJPIDLS(const TArray<FFBIKLinkData>& InLinkData, 
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		const Eigen::MatrixXf& InJacobianMatrix, 
		Eigen::MatrixXf& InOutAngleDerivativeMatrix, 
		FCalculateTargetVectorDelegate OnCalculateTargetVector, 
		const FSolverParameter& SolverParam);

	/* Create Angle PArtial Derivatives using Jacobian Transpose */
	bool CreateAnglePartialDerivativesUsingJT(const TArray<FFBIKLinkData>& InLinkData,
		const TMap<int32, FFBIKEffectorTarget>& InEndEffectors,
		const Eigen::MatrixXf& InJacobianMatrix,
		Eigen::MatrixXf& InOutAngleDerivativeMatrix,
		FCalculateTargetVectorDelegate OnCalculateTargetVector,
		const FSolverParameter& SolverParam);
};
