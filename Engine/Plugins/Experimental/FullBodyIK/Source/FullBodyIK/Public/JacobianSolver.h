// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JacobianIK.h"
#include "Math/Transform.h"

#define UE_API FULLBODYIK_API

using namespace JacobianIK;

/** 
 * Post processor for each iteration after resolving the pose
 * This will be used for Applying Constraints, but if you have more you want to do, you can customize this
 */
DECLARE_DELEGATE_OneParam(FPostProcessDelegateForIteration, TArray<FFBIKLinkData>& /*InOutLinkData*/);

/**
 * This is debug structure for editing time
 */
struct FJacobianDebugData
{
	TArray<FFBIKLinkData>	LinkData;

	// should match # of effectors
	TArray<FTransform>	TargetVectorSources;
	TArray<FVector>		TargetVectors;
};

/** 
 * Jacobian Solver Base Class
 * 
 * This does support two solvers
 * 1. Jacobian Transpose
 * 2. Jacobian Pseudo Inverse Damped Least Square (JPIDLS)
 * 
 * By default, we use JPIDLS, but it is cheaper to use Jacobian Transpose at the cost of solver
 */
class FJacobianSolverBase
{
public:
	UE_API FJacobianSolverBase();
	virtual ~FJacobianSolverBase() = default;

	void SetPostProcessDelegateForIteration(FPostProcessDelegateForIteration InDelegate)
	{
		OnPostProcessDelegateForIteration = InDelegate;
	}

	void ClearPostProcessDelegateForIteration()
	{
		OnPostProcessDelegateForIteration.Unbind();
	}

	UE_API bool SolveJacobianIK(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors, const FSolverParameter& InSolverParameter, int32 IterationCount = 30, float Tolerance = 1.f, TArray<FJacobianDebugData>* DebugData = nullptr);
private:

	// for reusing, and not reallocating memory
	Eigen::MatrixXf JacobianMatrix;
	Eigen::MatrixXf AnglePartialDerivatives;

	/* Solver generic APIs  - private APIs to be used internally */
	UE_API bool RunJacobianIK(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors, const FSolverParameter& InSolverParameter, float NormalizedIterationProgress);
	UE_API void UpdateClampMag(const TArray<FFBIKLinkData>& InLinkData, TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const;

	UE_API FVector ComputePartialDerivative(const FFBIKLinkData& InLinkData, bool bPositionChange, int32 LinkComponentIdx, const FFBIKLinkData& InEffectorLinkData, bool PositionTarget, const FSolverParameter& InSolverParameter);
	UE_API FVector ComputeTargetVector(const FFBIKLinkData& InEffectorLink, const FFBIKEffectorTarget& InEffectorData, bool bPositionTarget, const FSolverParameter& SolverParam);

	UE_API FQuat GetDeltaRotation(const FFBIKLinkData& LinkData, int32& OutPartialDerivativesIndex) const;
	UE_API FVector GetDeltaPosition(const FFBIKLinkData& LinkData, int32& OutPartialDerivativesIndex) const;

	UE_API FVector ClampMag(const FVector& ToTargetVector, float Length) const;
	UE_API void UpdateLinkData(TArray<FFBIKLinkData>& InOutLinkData, const Eigen::MatrixXf& InAnglePartialDerivatives) const;
	UE_API bool DidConverge(const TArray<FFBIKLinkData>& InLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors, float Tolerance, int32 Count) const;

protected:

	/* Initialize Solver - allows users to modify data before running the base solver - Check Below*/
	virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const {};
	/* Pre Solve - for each iteration, they run PreSolve. For each iteration, they may require to update info based on previous solved pose */
	virtual void PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const {};
	
protected:
	FCalculatePartialDerivativesDelegate OnCalculatePartialDerivativesDelegate;
	FCalculateTargetVectorDelegate OnCalculateTargetVectorDelegate;
	FPostProcessDelegateForIteration OnPostProcessDelegateForIteration;
};

/*
 * Generally each solver is categorized by 
 * 
 * 1. What type of target motion  : positional target (arm IK) or/and rotational target (look at)
 * 2. What type of joint motion : angular (joint rotates) or/and linear (joint translate)
 * 
 * The solver name indicates what target they support, and what motion it supports 
 */

 /*
  *  This support positional target using 3 different axis (default coordinate in world space - xyz axis)
  */
class FJacobianSolver_PositionTarget_3DOF : public FJacobianSolverBase
{
public: 
	UE_API FJacobianSolver_PositionTarget_3DOF();

protected:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
};

/*
 *  This support positional target using quaternion instead of 3 world axis
 *  This creates rotation axis using effector target of known (https://cseweb.ucsd.edu/classes/sp16/cse169-a/slides/CSE169_09.pdf)

 */
class FJacobianSolver_PositionTarget_Quat : public FJacobianSolverBase
{
public: 
	UE_API FJacobianSolver_PositionTarget_Quat();

private:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
	UE_API virtual void PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const override;
};

/*
 *  This support rotation target using quaternion instead of 3 world axis
 *  This creates rotation axis using effector target of known (https://cseweb.ucsd.edu/classes/sp16/cse169-a/slides/CSE169_09.pdf)
 */
class FJacobianSolver_RotationTarget_Quat : public FJacobianSolverBase
{
public:
	UE_API FJacobianSolver_RotationTarget_Quat();

private:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
	UE_API virtual void PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const override;
};

/*
 *  This support rotational target using 3 different axis (default coordinate in world space - xyz axis)
 */
class FJacobianSolver_RotationTarget_3DOF : public FJacobianSolver_PositionTarget_3DOF
{
public:
	UE_API FJacobianSolver_RotationTarget_3DOF();

private:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
};

/*
 *  This support positional and rotational target using 3 different axis (default coordinate in world space - xyz axis)
 */
class FJacobianSolver_PositionRotationTarget_3DOF : public FJacobianSolver_PositionTarget_3DOF
{
public:
	UE_API FJacobianSolver_PositionRotationTarget_3DOF();

private:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
};

/*
 *  This support position/rotation target using quaternion instead of 3 world axis
 *  This creates rotation axis using effector target of known (https://cseweb.ucsd.edu/classes/sp16/cse169-a/slides/CSE169_09.pdf)
 */
class FJacobianSolver_PositionRotationTarget_Quat : public FJacobianSolver_PositionTarget_Quat
{
public:
	UE_API FJacobianSolver_PositionRotationTarget_Quat();

private:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
	UE_API virtual void PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const override;
};

/*
 *  This support positional target using 3 different axis by translating joint (default coordinate in world space - xyz axis)
 */
class FJacobianSolver_PositionTarget_3DOF_Translation : public FJacobianSolver_PositionTarget_3DOF
{
public:
	UE_API FJacobianSolver_PositionTarget_3DOF_Translation();

protected:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
};

/*
 *  This support positional/rotational target using custom frame provided by user. If you want to create custom frame for stiffness.
 */
class FJacobianSolver_PositionRotationTarget_LocalFrame : public FJacobianSolverBase
{
public:
	UE_API FJacobianSolver_PositionRotationTarget_LocalFrame();

protected:
	UE_API virtual void InitializeSolver(TArray<FFBIKLinkData>& InOutLinkData, TMap<int32, FFBIKEffectorTarget>& InOutEndEffectors) const override;
	UE_API virtual void PreSolve(TArray<FFBIKLinkData>& InOutLinkData, const TMap<int32, FFBIKEffectorTarget>& InEndEffectors) const override;
};

#undef UE_API
