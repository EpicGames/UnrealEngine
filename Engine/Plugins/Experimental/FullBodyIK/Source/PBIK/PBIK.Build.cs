// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PBIK : ModuleRules
{
	public PBIK(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"RigVM",
				"ControlRig",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
			}
			);

		if (Target.bBuildEditor == true)
		{
			PublicDependencyModuleNames.Add("ControlRigDeveloper");
			PublicDependencyModuleNames.Add("ControlRigEditor");

			PrivateDependencyModuleNames.Add("Engine");
			PrivateDependencyModuleNames.Add("AssetTools");
			PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("RigVMDeveloper");
			PrivateDependencyModuleNames.Add("RigVMEditor");
		}
	}
}
