// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/PBIKBody.h"
#include "Core/PBIKConstraint.h"
#include "Core/PBIKSolver.h"

namespace PBIK
{

FBone::FBone(
	const FName InName,
	const int& InParentIndex,		// must pass -1 for root of whole skeleton
	const FVector& InOrigPosition,
	const FQuat& InOrigRotation,
	bool bInIsSolverRoot)
{
	Name = InName;
	ParentIndex = InParentIndex;
	Position = InOrigPosition;
	Rotation = InOrigRotation;
	bIsSolverRoot = bInIsSolverRoot;
}

bool FBone::HasChild(const FBone* Bone)
{
	for (const FBone* Child : Children)
	{
		if (Bone->Name == Child->Name)
		{
			return true;
		}
	}

	return false;
}

void FBone::UpdateFromInputs()
{
	if (!bIsPartOfSolvedBranch || !Parent)
	{
		return;
	}

	const FQuat ParentRotInv = Parent->Rotation.Inverse();
	LocalPositionFromInput = ParentRotInv * (Position - Parent->Position);
	LocalRotationFromInput = ParentRotInv * Rotation;
	Length = LocalPositionFromInput.Size();
}

FRigidBody::FRigidBody(FBone* InBone)
{
	Bone = InBone;
	J = FBoneSettings();
}

void FRigidBody::Initialize(const FBone* SolverRoot)
{
	// calculate transform and mass of body based on the skeleton
	UpdateTransformAndMassFromBones();
	
	// calculate num bones distance to root
	NumBonesToRoot = 0;
	const FBone* Parent = Bone;
	while (Parent && Parent != SolverRoot)
	{
		NumBonesToRoot += 1;
		Parent = Parent->Parent;
	}
}

void FRigidBody::UpdateFromInputs(const FPBIKSolverSettings& Settings)
{
	UpdateTransformAndMassFromBones();

	// update InvMass based on global mass multiplier
	constexpr float MinMass = 0.5f; // prevent mass ever hitting zero
	InvMass = 1.0f / FMath::Max(MinMass,(Mass * Settings.MassMultiplier * GLOBAL_UNITS));

	SolverSettings = &Settings;
}

void FRigidBody::UpdateTransformAndMassFromBones()
{
	FVector Centroid = Bone->Position;
	Mass = 0.0f;
	for(const FBone* Child : Bone->Children)
	{
		Centroid += Child->Position;
		Mass += (Bone->Position - Child->Position).Size();
	}
	Centroid = Centroid * (1.0f / (Bone->Children.Num() + 1.0f));

	Position = InputPosition = Centroid;
	Rotation = InitialRotation = Bone->Rotation;
	BoneLocalPosition = Bone->Rotation.Inverse() * (Bone->Position - Centroid);
}

int FRigidBody::GetNumBonesToRoot() const
{ 
	return NumBonesToRoot; 
}

FRigidBody* FRigidBody::GetParentBody() const
{
	if (Bone && Bone->Parent)
	{
		return Bone->Parent->Body;
	}

	return nullptr;
}

bool FRigidBody::IsPositionLocked() const
{
	return bIsLockedBySubSolve || InvMass <= SMALL_NUMBER || FMath::IsNearlyEqual(J.PositionStiffness, 1.0f);
}

bool FRigidBody::IsRotationLocked() const
{
	return bIsLockedBySubSolve || InvMass <= SMALL_NUMBER || FMath::IsNearlyEqual(J.RotationStiffness, 1.0f);
}

void FRigidBody::ApplyPushToRotateBody(const FVector& Push, const FVector& Offset)
{
	if (IsRotationLocked())
	{
		return; // rotation of this body is disabled
	}
	
	// equation 8 in "Detailed Rigid Body Simulation with XPBD"
	const float Lambda = InvMass * (1.0f - J.RotationStiffness) * SolverSettings->OverRelaxation;
	const FVector Omega = Lambda * FVector::CrossProduct(Offset, Push);
	const FQuat DeltaQ(Omega.X, Omega.Y, Omega.Z, 0.0f);
	ApplyRotationDelta(DeltaQ);
}

void FRigidBody::ApplyPositionDelta(const FVector& DeltaP)
{
	if (IsPositionLocked())
	{
		return; // translation of this body is disabled
	}
	
	Position += DeltaP * (1.0f - J.PositionStiffness) * SolverSettings->OverRelaxation;
}

void FRigidBody::ApplyRotationDelta(const FQuat& DeltaQ)
{
	if (IsRotationLocked())
	{
		return; // rotation of this body is disabled
	}

	// limit rotation each iteration
	FQuat ClampedDQ = DeltaQ;
	const float MaxPhi = FMath::DegreesToRadians(SolverSettings->MaxAngle);
	const float Phi = DeltaQ.Size();
	if (Phi > MaxPhi)
	{
		ClampedDQ *= MaxPhi / Phi;
	}
	
	Rotation += (ClampedDQ * Rotation) * 0.5f;
}
} // namespace
