// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/PBIKConstraint.h"
#include "Core/PBIKBody.h"
#include "Core/PBIKSolver.h"

namespace PBIK
{
	
FJointConstraint::FJointConstraint(FRigidBody* InA, FRigidBody* InB)
{
	A = InA;
	B = InB;

	const FVector PinPoint = B->Bone->Position;
	PinPointLocalToA = A->Rotation.Inverse() * (PinPoint - A->Position);
	PinPointLocalToB = B->Rotation.Inverse() * (PinPoint - B->Position);

	XOrig = B->InitialRotation * FVector(1, 0, 0);
	YOrig = B->InitialRotation * FVector(0, 1, 0);
	ZOrig = B->InitialRotation * FVector(0, 0, 1);

	AngleX = AngleY = AngleZ = 0;

	bInFinalPass = false;
}
	
void FJointConstraint::Solve(const FPBIKSolverSettings& Settings)
{
	// early out if both bodies are locked (joint cannot do anything)
	if (A->bIsLockedBySubSolve && B->bIsLockedBySubSolve)
	{
		return;
	}

	// calc inv mass of body A and B
	const float W = A->InvMass + B->InvMass;
	if (FMath::IsNearlyZero(W))
	{
		return; // no correction can be applied on full locked configuration
	}

	// get pos correction to use for rotation
	FVector OffsetA;
	FVector OffsetB;
	FVector Correction = GetPositionCorrection(OffsetA, OffsetB);
	FVector N;
	float C;
	Correction.ToDirectionAndLength(N, C);
	const FVector CorrectDir = N * (C * (1.0f / W));
	const FVector CorrectA = CorrectDir * A->InvMass;
	const FVector CorrectB = -CorrectDir * B->InvMass;

	// apply rotation from correction at pin point to both bodies
	A->ApplyPushToRotateBody(CorrectA, OffsetA);
	B->ApplyPushToRotateBody(CorrectB, OffsetB);

	// enforce joint limits
	UpdateJointLimits();
	
	// apply position correction
	A->ApplyPositionDelta(CorrectA);
	B->ApplyPositionDelta(CorrectB);

	// must normalize due to infinitesimal quaternion addition
	A->Rotation.Normalize();
	B->Rotation.Normalize();
}

void FJointConstraint::RemoveStretch(const float Percent)
{
	const FVector PinPointOnA = A->Position + A->Rotation * PinPointLocalToA;
	const FVector PinPointOnB = B->Position + B->Rotation * PinPointLocalToB;
	B->Position -= (PinPointOnB - PinPointOnA) * Percent;
}

void FJointConstraint::FinalPass()
{
	if (B->bIsLockedBySubSolve)
	{
		return;
	}
	
	bInFinalPass = true;
	UpdateJointLimits();
	bInFinalPass = false;
}

void FJointConstraint::UpdateFromInputs()
{
	const FVector PinPoint = B->Bone->Position;
	PinPointLocalToA = A->Rotation.Inverse() * (PinPoint - A->Position);
	PinPointLocalToB = B->Rotation.Inverse() * (PinPoint - B->Position);
}

FVector FJointConstraint::GetPositionCorrection(FVector& OutBodyToA, FVector& OutBodyToB) const
{
	OutBodyToA = A->Rotation * PinPointLocalToA;
	OutBodyToB = B->Rotation * PinPointLocalToB;
	const FVector PinPointOnA = A->Position + OutBodyToA;
	const FVector PinPointOnB = B->Position + OutBodyToB;
	return (PinPointOnB - PinPointOnA);
}

void FJointConstraint::ApplyRotationCorrection(FQuat DeltaQ) const
{
	A->ApplyRotationDelta(DeltaQ * (1.0f - A->J.RotationStiffness));
	B->ApplyRotationDelta(DeltaQ * -1.0f * (1.0f - B->J.RotationStiffness));
}

void FJointConstraint::UpdateJointLimits()
{
	FBoneSettings& J = B->J;
	
	// no limits at all
	if (J.X == ELimitType::Free &&
		J.Y == ELimitType::Free &&
		J.Z == ELimitType::Free)
	{
		return;
	}

	// force max angles > min angles
	J.MaxX = J.MaxX < J.MinX ? J.MinX + 1 : J.MaxX;
	J.MaxY = J.MaxY < J.MinY ? J.MinY + 1 : J.MaxY;
	J.MaxZ = J.MaxZ < J.MinZ ? J.MinZ + 1 : J.MaxZ;

	// determine which axes are explicitly or implicitly locked (limited with very small allowable angle)
	constexpr float LockThresholdAngle = 2.0f; // degrees
	const bool bLockX = (J.X == ELimitType::Locked) || (J.X == ELimitType::Limited && ((J.MaxX - J.MinX) < LockThresholdAngle));
	const bool bLockY = (J.Y == ELimitType::Locked) || (J.Y == ELimitType::Limited && ((J.MaxY - J.MinY) < LockThresholdAngle));
	const bool bLockZ = (J.Z == ELimitType::Locked) || (J.Z == ELimitType::Limited && ((J.MaxZ - J.MinZ) < LockThresholdAngle));

	// when locked on THREE axes, consider the bone FIXED
	if (bLockX && bLockY && bLockZ)
	{
		const FQuat OrigOffset = A->InitialRotation * B->InitialRotation.Inverse();
		const FQuat CurrentOffset = A->Rotation * B->Rotation.Inverse();
		const FQuat DeltaOffset = OrigOffset * CurrentOffset.Inverse();
		if (bInFinalPass)
		{
			B->Rotation = DeltaOffset.Inverse() * B->Rotation;
			return;
		}
		
		const FQuat DeltaQ = FQuat(DeltaOffset.X, DeltaOffset.Y, DeltaOffset.Z, 0.0f);
		ApplyRotationCorrection(DeltaQ);
		return; // fixed bones cannot move at all
	}

	// when locked on TWO axes, consider the bone a HINGE (mutually exclusive)
	const bool bXHinge = !bLockX && bLockY && bLockZ;
	const bool bYHinge = bLockX && !bLockY && bLockZ;
	const bool bZHinge = bLockX && bLockY && !bLockZ;

	// apply hinge corrections
	if (bXHinge)
	{
		UpdateLocalRotateAxes(true, false, false);
		RotateToAlignAxes(XA, XB);
	}
	else if (bYHinge)
	{
		UpdateLocalRotateAxes(false, true, false);
		RotateToAlignAxes(YA, YB);
	}
	else if (bZHinge)
	{
		UpdateLocalRotateAxes(false, false, true);
		RotateToAlignAxes(ZA, ZB);
	}

	// enforce min/max angles
	const bool bLimitX = J.X == ELimitType::Limited && !bLockX;
	const bool bLimitY = J.Y == ELimitType::Limited && !bLockY;
	const bool bLimitZ = J.Z == ELimitType::Limited && !bLockZ;
	if (bLimitX || bLimitY || bLimitZ)
	{
		DecomposeRotationAngles();
	}

	if (bLimitX)
	{
		RotateWithinLimits(J.MinX, J.MaxX, AngleX, XA, ZBProjOnX, ZA);
	}

	if (bLimitY)
	{
		RotateWithinLimits(J.MinY, J.MaxY, AngleY, -YA, ZBProjOnY, ZA);
	}

	if (bLimitZ)
	{
		RotateWithinLimits(J.MinZ, J.MaxZ, AngleZ, ZA, YBProjOnZ, YA);
	}
}

void FJointConstraint::RotateToAlignAxes(const FVector& AxisA, const FVector& AxisB) const
{
	if (bInFinalPass)
	{
		const FQuat DeltaQ = FQuat::FindBetweenVectors(AxisB, AxisA);
		B->Rotation = DeltaQ * B->Rotation;
		return;
	}
	
	const FVector ACrossB = FVector::CrossProduct(AxisA, AxisB);
	FVector Axis;
	float Magnitude;
	ACrossB.ToDirectionAndLength(Axis, Magnitude);
	const float EffectiveInvMass = FVector::DotProduct(Axis, Axis); // Todo incorporate Body Mass here?
	if (EffectiveInvMass < KINDA_SMALL_NUMBER)
	{
		return;
	}
	const float DeltaLambda = Magnitude / EffectiveInvMass;
	const FVector Push = Axis * DeltaLambda;
	const FQuat DeltaQ = FQuat(Push.X, Push.Y, Push.Z, 0.0f);
	ApplyRotationCorrection(DeltaQ);
}

void FJointConstraint::RotateWithinLimits(
	float MinAngle,
	float MaxAngle,
	float CurrentAngle,
	FVector RotAxis,
	FVector CurVec,
	FVector RefVec) const
{
	const bool bBeyondMin = CurrentAngle < MinAngle;
	const bool bBeyondMax = CurrentAngle > MaxAngle;
	if (!(bBeyondMin || bBeyondMax))
	{
		return;
	}
	const float TgtAngle = bBeyondMin ? MinAngle : MaxAngle;
	const FQuat TgtRot = FQuat(RotAxis, FMath::DegreesToRadians(TgtAngle));
	const FVector TgtVec = TgtRot * RefVec;
	RotateToAlignAxes(TgtVec, CurVec);
}

void FJointConstraint::UpdateLocalRotateAxes(bool bX, bool bY, bool bZ)
{
	const FQuat ARot = A->Rotation * A->InitialRotation.Inverse();
	const FQuat BRot = B->Rotation * B->InitialRotation.Inverse();

	if (bX)
	{
		XA = ARot * XOrig;
		XB = BRot * XOrig;
	}

	if (bY)
	{
		YA = ARot * YOrig;
		YB = BRot * YOrig;
	}

	if (bZ)
	{
		ZA = ARot * ZOrig;
		ZB = BRot * ZOrig;
	}
}

void FJointConstraint::DecomposeRotationAngles()
{
	const FQuat ARot = A->Rotation * A->InitialRotation.Inverse();
	const FQuat BRot = B->Rotation * B->InitialRotation.Inverse();

	XA = ARot * XOrig;
	YA = ARot * YOrig;
	ZA = ARot * ZOrig;
	XB = BRot * XOrig;
	YB = BRot * YOrig;
	ZB = BRot * ZOrig;

	ZBProjOnX = FVector::VectorPlaneProject(ZB, XA).GetSafeNormal();
	ZBProjOnY = FVector::VectorPlaneProject(ZB, YA).GetSafeNormal();
	YBProjOnZ = FVector::VectorPlaneProject(YB, ZA).GetSafeNormal();

	AngleX = SignedAngleBetweenNormals(ZA, ZBProjOnX, XA);
	AngleY = -SignedAngleBetweenNormals(ZA, ZBProjOnY, YA);
	AngleZ = SignedAngleBetweenNormals(YA, YBProjOnZ, ZA);
}

float FJointConstraint::SignedAngleBetweenNormals(
	const FVector& From, 
	const FVector& To, 
	const FVector& Axis) const
{
	const float FromDotTo = FVector::DotProduct(From, To);
	const float Angle = FMath::RadiansToDegrees(FMath::Acos(FromDotTo));
	const FVector Cross = FVector::CrossProduct(From, To);
	const float Dot = FVector::DotProduct(Cross, Axis);
	return Dot >= 0 ? Angle : -Angle;
}

FPinConstraint::FPinConstraint(FRigidBody* InBody, FBone* InPinBone)
{
	PinBone = InPinBone;
	Body = InBody;
	PinPointLocalToA = Body->Rotation.Inverse() * (PinBone->Position - Body->Position);
	
	GoalPosition = InPinBone->Position;
	GoalRotation = InPinBone->Rotation;
}

void FPinConstraint::Solve(const FPBIKSolverSettings& Settings)
{
	if (!bEnabled || (Alpha <= KINDA_SMALL_NUMBER))
	{
		return;
	}

	// get positional correction for rotation
	FVector AToPinPoint;
	FVector Correction = GetPositionCorrection(AToPinPoint);
		
	// rotate body from alignment of pin points
	Body->ApplyPushToRotateBody(Correction, AToPinPoint);

	// apply positional correction to Body to align with target (after rotation)
	// (applying directly without considering PositionStiffness because PinConstraints need
	// to precisely pull the attached body to achieve convergence)
	Correction = GetPositionCorrection(AToPinPoint);
	Body->Position += Correction * Settings.OverRelaxation; 
}

void FPinConstraint::SetGoal(const FVector& InGoalPosition, const FQuat& InGoalRotation, const float InAlpha)
{
	GoalPosition = InGoalPosition;
	GoalRotation = InGoalRotation;
	Alpha = InAlpha;
}

FVector FPinConstraint::GetCurrentPinPoint() const
{
	return Body->Position + (Body->Rotation * PinPointLocalToA);
}

void FPinConstraint::UpdateFromInputs()
{
	PinPointLocalToA = Body->Rotation.Inverse() * (PinBone->Position - Body->Position);
}

FVector FPinConstraint::GetPositionCorrection(FVector& OutBodyToPinPoint) const
{
	OutBodyToPinPoint = Body->Rotation * PinPointLocalToA;
	const FVector PinPoint = Body->Position + OutBodyToPinPoint;
	return (GoalPosition - PinPoint) * Alpha;
}

} // namespace

