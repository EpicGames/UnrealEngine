// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/PBIKSolver.h"
#include "Core/PBIKBody.h"
#include "Core/PBIKConstraint.h"
#include "Core/PBIKDebug.h"
#include "PBIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PBIKSolver)

namespace PBIK
{
	FEffector::FEffector(FBone* InBone)
	{
		check(InBone);
		Bone = InBone;
		SetGoal(Bone->Position, Bone->Rotation, Settings);
	}

	void FEffector::SetGoal(
		const FVector& InPositionGoal,
		const FQuat& InRotationGoal,
		const FEffectorSettings& InSettings)
	{
		Settings = InSettings;
		PositionGoal = InPositionGoal;
		RotationGoal = InRotationGoal;
	}

	void FEffector::UpdateChainStates()
	{
		// NOTE: this function must be called AFTER InitBodies() due to it's reliance on the FBody.bIsSubRoot flag
		
		ChainRootBody = nullptr;
		const int32 MaxDepth = Settings.ChainDepth;
		const FBone* Parent = Bone->Parent;
		const FBone* Child = Bone;
		int32 Depth = 1;
		while (Parent)
		{
			if (!Parent->bIsSolved)
			{
				break; // this only happens when effector is on solver root
			}

			// if user specified a custom chain depth, then use it.
			// otherwise halt when we hit a sub root (a branch) or root of the solver
			const bool bIsAtAnyRoot = Parent->bIsSubRoot || Parent->bIsSolverRoot;
			const bool bIsAtChainRoot = MaxDepth > 0 ? Depth > MaxDepth || bIsAtAnyRoot : bIsAtAnyRoot;
			if (bIsAtChainRoot)
			{
				ChainRootBody = Child->Body;
				break;
			}
			
			Parent->Body->bIsPartOfSubChain = true;
			++Depth;
			Child = Parent;
			Parent = Parent->Parent;
		}

		ChainDepthInitializedWith = Settings.ChainDepth;
	}

	void FEffector::UpdateFromInputs(const FBone& InSolverRoot, const bool bAllowStretch)
	{
		// update original transform based on input bone pose
		PositionOrig = Bone->Position;
		RotationOrig = Bone->Rotation;
		
		// update final position/rotation by blending from input bone pose to goal by alpha
		PositionFinal = FMath::Lerp(PositionOrig, PositionGoal, Settings.PositionAlpha);
		RotationFinal = FQuat::Slerp(RotationOrig, RotationGoal, Settings.RotationAlpha);

		// if the solver root is pinned, we clamp the effector position so that it cannot pull excessively,
		// this prevents extreme constraint forces that cause solve instability
		if (FMath::IsNearlyZero(InSolverRoot.Body->InvMass) && !bAllowStretch)
		{
			const float DistToSolveRootAlongBones = CalculateDistanceToSolveRootAlongBones();
			PositionFinal = InSolverRoot.Position + (PositionFinal - InSolverRoot.Position).GetClampedToMaxSize(DistToSolveRootAlongBones);
		}

		// set pin constraint final transform
		Pin->SetGoal(PositionFinal, RotationFinal, Settings.StrengthAlpha);
		
		// update length of chain this effector controls in the input pose
		DistToChainRootInInputPose = CalculateDistanceToChainRoot();

		// update distances to chain root (along bones)
		if (ChainRootBody)
		{
			DistancesFromEffector.Reset();
			DistToChainRootAlongBones = 0.0;
			const FBone* Parent = Bone;
			while (true)
			{
				DistancesFromEffector.Add(DistToChainRootAlongBones);

				if (!Parent || Parent->Body == ChainRootBody)
				{
					break; // stop before advancing past the chain root
				}
				
				DistToChainRootAlongBones += Parent->Length;
				Parent = Parent->Parent;
			}
		}
	}

	float FEffector::CalculateDistanceToChainRoot() const
	{
		// calculates distance from the bone on this effector to the parent sub-root (chain root)
		if (ChainRootBody)
		{
			const FEffector* ParentEffector = ChainRootBody->Effector;
			const FVector ParentSubRootPosition = ParentEffector ? ParentEffector->PositionOrig : ChainRootBody->Position;
			return (ParentSubRootPosition - Bone->Position).Size();
		}

		return 0.f;
	}

	float FEffector::CalculateDistanceToSolveRootAlongBones() const
	{
		FBone* CurrentBone = Bone;
		float DistanceTotal = 0.0f; 
		while (!CurrentBone->bIsSolverRoot)
		{
			DistanceTotal += CurrentBone->Length;
			CurrentBone = CurrentBone->Parent;
		}
		return DistanceTotal;
	}

	void FEffector::ApplyPreferredAngles() const
	{
		// optionally apply a preferred angle to give solver a hint which direction to favor
		// apply amount of preferred angle proportional to the amount this sub-limb is squashed

		// can't squash root chain
		if (!ChainRootBody)
		{
			return; 
		}

		// can't squash chain with zero length already
		if (DistToChainRootInInputPose <= SMALL_NUMBER)
		{
			return;
		}

		// we have to be careful here when calculating the distance to the parent sub-root.
		// if the parent sub-root is attached to an effector, use the effector's position
		// otherwise use the current position of the FRigidBody
		const FEffector* ParentEffector = ChainRootBody->Effector;
		const FVector ParentSubRootPosition = ParentEffector ? ParentEffector->PositionFinal : ChainRootBody->Position;
		const float DistToSubRootCurrent = (ParentSubRootPosition - PositionFinal).Size();
		if (DistToSubRootCurrent >= DistToChainRootInInputPose)
		{
			return; // limb is stretched
		}

		// amount squashed (clamped to scaled original length)
		const float DeltaSquash = DistToChainRootInInputPose - DistToSubRootCurrent;
		float SquashPercent = DeltaSquash / DistToChainRootInInputPose;
		SquashPercent = PBIK::CircularEaseOut(SquashPercent);
		if (FMath::IsNearlyZero(SquashPercent))
		{
			return; // limb not squashed enough
		}

		// iterate over all the bodies in the chain controlled by this effector...
		const FBone* Parent = Bone->Parent;
		while (Parent && Parent->bIsSolved)
		{
			FRigidBody* Body = Parent->Body;
			if (Body->J.bUsePreferredAngles)
			{
				const FRigidBody* CurrentParent = Body->GetParentBody();
				FQuat CurrentLocalRotation = CurrentParent ? CurrentParent->Rotation.Inverse() * Body->Rotation : Body->Rotation;
				FQuat InitialLocalRotation = CurrentParent ? CurrentParent->InitialRotation.Inverse() * Body->InitialRotation : Body->InitialRotation;
				FQuat LocalOffsetRotation = CurrentLocalRotation * InitialLocalRotation.Inverse();
				const FRotator AnglesFromInput = LocalOffsetRotation.Rotator();
				const FRotator PreferredAngles = Body->J.PreferredAngles * SquashPercent;

				auto CalcDeltaAngle = [](double PreferredAngle, double InputAngle) -> double
				{
					if (FMath::Abs(PreferredAngle) <= SMALL_NUMBER)
					{
						return 0.f; // apply no delta, preferred angle was unset at 0
					}

					// if the target angle is greater than the input pose, return a delta
					if (FMath::Abs(PreferredAngle) > FMath::Abs(InputAngle))
					{
						return PreferredAngle - InputAngle;
					}

					// body is already bent more than the preferred angle
					return 0.f;
				};

				FRotator LocalDeltaAngles;
				LocalDeltaAngles.Pitch = CalcDeltaAngle(PreferredAngles.Pitch, AnglesFromInput.Pitch);
				LocalDeltaAngles.Yaw = CalcDeltaAngle(PreferredAngles.Yaw, AnglesFromInput.Yaw);
				LocalDeltaAngles.Roll = CalcDeltaAngle(PreferredAngles.Roll, AnglesFromInput.Roll);
				Body->Rotation = Body->Rotation * LocalDeltaAngles.Quaternion();
				Body->Rotation.Normalize();
			}

			if (Parent == ChainRootBody->Bone)
			{
				return;
			}

			Parent = Parent->Parent;
		}
	}
} // namespace

void FPBIKSolver::Solve(const FPBIKSolverSettings& InSolverSettings)
{
	SCOPE_CYCLE_COUNTER(STAT_PBIK_Solve);

	using PBIK::FEffector;
	using PBIK::FRigidBody;
	using PBIK::FBone;
	using PBIK::FBoneSettings;

	// don't run until properly initialized
	if (!Initialize())
	{
		return;
	}

	// initialize local bone transforms
	// this has to be done every tick because incoming animation can modify these
	// even the LocalPosition has to be updated in case translation is animated
	for (FBone& Bone : Bones)
	{
		Bone.UpdateFromInputs();
	}

	// update Bodies with new bone positions from incoming pose and solver settings
	for (FRigidBody& Body : Bodies)
	{
		Body.UpdateFromInputs(InSolverSettings);
	}

	// optionally pin root in-place (convenience, does not require an effector)
	if (InSolverSettings.RootBehavior == EPBIKRootBehavior::PinToInput)
	{
		// we can safely override the mass here to pin the root body
		// if the user switches to a different root mode, the call to Body.UpdateFromInputs() will restore the correct InvMass for the body
		SolverRoot->Body->InvMass = 0.0f;
	}

	// lazily updates effector chain depths (IFF settings change at runtime)
	constexpr bool bForceUpdate = false;
	UpdateEffectorDepths(bForceUpdate);

	// blend effectors by Alpha, update pin goals and update effector dist to root
	for (FEffector& Effector : Effectors)
	{
		Effector.UpdateFromInputs(*SolverRoot, InSolverSettings.bAllowStretch);
	}

	// update constraints with new local pin locations relative to (possibly) translated joint positions
	for (const TSharedPtr<PBIK::FConstraint>& Constraint : Constraints)
	{
		Constraint->UpdateFromInputs();
	}
		
	// do all constraint solving
	UpdateBodies(InSolverSettings);

	// now that bodies are posed, update bone transforms
	UpdateBonesFromBodies();
}

void FPBIKSolver::UpdateBodies(const FPBIKSolverSettings& Settings)
{
	using PBIK::FRigidBody;

	// apply large scale gross movement to entire skeleton based on effector deltas
	// (this creates a better pose from which to start the constraint solving)
	ApplyRootPrePull(Settings.RootBehavior, Settings.PrePullRootSettings);

	// pre-rotate bones towards preferred angles when effector limb is squashed
	ApplyPreferredAngles();

	// pull each effector's chain towards itself
	ApplyPullChainAlpha(Settings.GlobalPullChainAlpha);

	// run ALL constraint iterations
	SolveConstraints(Settings);
}

void FPBIKSolver::UpdateBonesFromBodies()
{
	using PBIK::FRigidBody;
	using PBIK::FBone;
	using PBIK::FEffector;

	// iterate over ALL bones in root-to-leaf order
	for (FBone& Bone : Bones)
	{
		// bone is located outside the solving branch in the hierarchy
		if (!Bone.bIsPartOfSolvedBranch)
		{
			continue;
		}

		// trivial leaf bones (children of simulated bones)
		if (!Bone.bIsSolved && Bone.Parent)
		{
			Bone.Position = Bone.Parent->Position + Bone.Parent->Rotation * Bone.LocalPositionFromInput;
			Bone.Rotation = Bone.Parent->Rotation * Bone.LocalRotationFromInput;
			continue;
		}
		
		// if bone is controlled by a body, then update it from the body local offset
		if (Bone.Body)
		{
			Bone.Position = Bone.Body->Position + Bone.Body->Rotation * Bone.Body->BoneLocalPosition;
			Bone.Rotation = Bone.Body->Rotation;
		}
		
		// if bone is controlled by an effector, we optionally apply the effector rotation
		if (Bone.EffectorIndex != INDEX_NONE)
		{
			// get the effector associated with this bone
			const FEffector& Effector = Effectors[Bone.EffectorIndex];
			
			// if effector is between other effectors, it will have an associated rigid body
			// so leave transform where body ended up after solve (applied above)
			// otherwise, snap position back to FK location
			if (!Bone.Body)
			{
				if (Bone.Parent)
				{
					Bone.Position = Bone.Parent->Position + Bone.Parent->Rotation * Bone.LocalPositionFromInput;
				}
				else
				{
					// effector was on the solver root
					Bone.Position = Effector.PositionFinal;
				}
			}

			// optionally pin rotation to that of effector
			if (FMath::IsNearlyEqual(Effector.Settings.PinRotation, 1.0f))
			{
				Bone.Rotation = Effector.RotationFinal;
			}
			else
			{
				// blend between bone rotation and effector rotation
				const FQuat RotationWithoutEffector = Bone.Parent ? Bone.Parent->Rotation * Bone.LocalRotationFromInput : Bone.Rotation; 
				const float RotAmount = FMath::Clamp(Effector.Settings.PinRotation, 0.0f, 1.0f);
				Bone.Rotation = FQuat::FastLerp(RotationWithoutEffector, Effector.RotationFinal, RotAmount).GetNormalized();
			}	
		}
	}
}

void FPBIKSolver::ApplyRootPrePull(const EPBIKRootBehavior RootBehavior, const FRootPrePullSettings& PrePullSettings)
{
	using PBIK::FEffector;
	using PBIK::FRigidBody;
	
	if (RootBehavior != EPBIKRootBehavior::PrePull)
	{
		return;
	}

	// calculate a "best fit" transform from deformed goal locations
	TArray<FVector, TInlineAllocator<8>> InitialPoints;
	TArray<FVector, TInlineAllocator<8>> CurrentPoints;
	InitialPoints.Reserve(Effectors.Num());
	CurrentPoints.Reserve(Effectors.Num());
	for (FEffector& Effector : Effectors)
	{
		InitialPoints.Add(Effector.PositionOrig);
		CurrentPoints.Add(Effector.PositionFinal);
	}

	// calculate "best fit" rotational delta from deformed effector configuration
	FVector InitialCentroid = FVector::ZeroVector;
	FVector CurrentCentroid = FVector::ZeroVector;
	const FQuat RotationOffset = GetRotationFromDeformedPoints(
		InitialPoints,
		CurrentPoints,
		InitialCentroid,
		CurrentCentroid);

	// do per-axis alpha blend for position offset
	FVector PositionOffset = CurrentCentroid - InitialCentroid;
	PositionOffset *= FVector(PrePullSettings.PositionAlphaX, PrePullSettings.PositionAlphaY, PrePullSettings.PositionAlphaZ);
	// alpha blend entire offset
	PositionOffset *= PrePullSettings.PositionAlpha;
	
	// do per-axis alpha blend for rotation offset
	const FVector Euler = RotationOffset.Euler() * FVector(PrePullSettings.RotationAlphaX, PrePullSettings.RotationAlphaY, PrePullSettings.RotationAlphaZ);
	FQuat FinalRotationOffset = FQuat::MakeFromEuler(Euler);
	// alpha blend the entire rotation offset
	FinalRotationOffset = FQuat::FastLerp(FQuat::Identity, FinalRotationOffset, PrePullSettings.RotationAlpha).GetNormalized();

	// move and rotate all bodies rigidly with the root
	const FRigidBody* RootBody = SolverRoot->Body;
	const FVector RootPosition = RootBody->Position;
	const FQuat RootRotationInv = RootBody->Rotation.Inverse();
	const FVector NewRootPosition = RootPosition + PositionOffset;
	const FQuat NewRootRotation = FinalRotationOffset * RootBody->Rotation;
	for (FRigidBody& Body : Bodies)
	{
		FVector BodyLocalPos = RootRotationInv * (Body.Position - RootPosition);
		FQuat LocalRot = RootRotationInv * Body.Rotation;
		Body.Position = NewRootPosition + (NewRootRotation * BodyLocalPos);
		Body.Rotation = NewRootRotation * LocalRot;
	}
}

void FPBIKSolver::ApplyPullChainAlpha(const float GlobalPullChainAlpha)
{
	using PBIK::FEffector;
	using PBIK::FBone;
	using PBIK::FRigidBody;

	if (FMath::IsNearlyZero(GlobalPullChainAlpha))
	{
		return;
	}
	
	for (FEffector& Effector : Effectors)
	{
		if (!Effector.ChainRootBody)
		{
			continue;
		}

		if (Effector.DistToChainRootAlongBones < SMALL_NUMBER)
		{
			continue;
		}

		if (Effector.Settings.PullChainAlpha < SMALL_NUMBER)
		{
			continue;
		}

		// get the current start point on the chain
		FRigidBody* RootBody = Effector.ChainRootBody;
		FVector ChainStartPosition = RootBody->Position + RootBody->Rotation * RootBody->BoneLocalPosition;

		// if the chain root has an effector we have to add it's offset here
		if (RootBody->Effector)
		{
			const FEffector* RootEffector = RootBody->Effector;
			FVector RootEffectorDelta = RootEffector->PositionFinal - RootEffector->PositionOrig;
			ChainStartPosition += RootEffectorDelta;
		}

		// get current chain vector
		FVector ChainVecCurrent;
		float ChainLenCurrent;
		(Effector.Pin->GetCurrentPinPoint() - ChainStartPosition).ToDirectionAndLength(ChainVecCurrent, ChainLenCurrent);

		// get new chain vector
		FVector ChainVecNew;
		float ChainLenNew;
		(Effector.PositionFinal - ChainStartPosition).ToDirectionAndLength(ChainVecNew, ChainLenNew);

		FQuat ChainDeltaRotation = FQuat::FindBetweenNormals(ChainVecCurrent, ChainVecNew);
		const float ChainDeltaAlpha = Effector.Settings.PullChainAlpha * Effector.Settings.StrengthAlpha * GlobalPullChainAlpha;
		ChainDeltaRotation = FQuat::FastLerp(FQuat::Identity, ChainDeltaRotation, ChainDeltaAlpha).GetNormalized();
		FVector ChainDeltaPosition = ChainVecNew * (ChainLenNew - ChainLenCurrent) * ChainDeltaAlpha;

		const FBone* Bone = Effector.Bone->Parent;
		int32 ChainIndex = 0;
		const float InvChainLength = 1.0f / Effector.DistToChainRootAlongBones;
		while (Bone)
		{
			// move body along with chain
			if (!Bone->Body->IsPositionLocked())
			{
				// move first to the rotated chain position
				const FVector BodyRelativeToChain = Bone->Body->Position - ChainStartPosition;
				const FVector RotatedBodyPos = ChainStartPosition + ChainDeltaRotation.RotateVector(BodyRelativeToChain);
				Bone->Body->Position = RotatedBodyPos;

				// translate along chain proportional to chain stretch amount
				const float Strength = 1.0f - (Effector.DistancesFromEffector[ChainIndex] * InvChainLength);
				Bone->Body->Position += ChainDeltaPosition * Strength;
			}

			// rotate body along with chain
			if (!Bone->Body->IsRotationLocked())
			{
				Bone->Body->Rotation = ChainDeltaRotation * Bone->Body->Rotation;
			}

			if (Bone->Body == RootBody)
			{
				break;
			}
			
			Bone = Bone->Parent;
			++ChainIndex;
		}
	}
}

void FPBIKSolver::ApplyPreferredAngles()
{
	// apply preferred angles to squashed effector chains
	using PBIK::FEffector;
	for (FEffector& Effector : Effectors)
	{
		Effector.ApplyPreferredAngles();
	}

	// preferred angles introduce stretch, so remove that to prevent biasing the first constraint iteration
	for (int32 C = Constraints.Num() - 1; C >= 0; --C)
	{
		constexpr float FullAmount = 1.0f;
		Constraints[C]->RemoveStretch(FullAmount);
	}
}

void FPBIKSolver::SolveConstraints(const FPBIKSolverSettings& InSolverSettings)
{
	using PBIK::FRigidBody;

	auto RunConstraintPass = [this, &InSolverSettings](int32 CurrentIteration, int32 MaxIterations)
	{
		// solve all constraints
		for (const TSharedPtr<PBIK::FConstraint>& Constraint : Constraints)
		{
			Constraint->Solve(InSolverSettings);
		}

		// on final iteration, run FinalPass
		if (CurrentIteration == MaxIterations - 1)
		{
			// iterate in root-to-leaf order to force exact rotation limits
			for (int32 C = Constraints.Num() - 1; C >= 0; --C)
			{
				Constraints[C]->FinalPass();
			}
		}
		
		// do post-pass to remove stretch
		if (!InSolverSettings.bAllowStretch)
		{
			const float StretchAmount = FMath::Min(1.0f, static_cast<float>(CurrentIteration+1) / (MaxIterations / 2.0f));
			// iterate in reverse order (root to leaf)
			for (int32 C = Constraints.Num() - 1; C >= 0; --C)
			{
				Constraints[C]->RemoveStretch(StretchAmount);
			}
		}
	};

	auto RunSubIterations = [this, &RunConstraintPass, &InSolverSettings]()
	{
		// run pre-pass where we lock everything not in a sub-chain
		if (!bHasSubChains || InSolverSettings.SubIterations <= 0)
		{
			return;
		}
		// first lock all the bodies not in a sub chain
		for (FRigidBody& Body : Bodies)
		{
			Body.bIsLockedBySubSolve = !Body.bIsPartOfSubChain;
		}

		// solve all constraints (locked constraints are skipped in this pass)
		for (int32  I = 0; I < InSolverSettings.SubIterations; ++I)
		{
			RunConstraintPass(I, InSolverSettings.SubIterations);
		}

		// unlock all bodies so we can proceed with normal constraint solve
		for (FRigidBody& Body : Bodies)
		{
			Body.bIsLockedBySubSolve = false;
		}
	};

	auto RunMainIterations = [this, &RunConstraintPass, &InSolverSettings]()
	{
		// run main constraint pass
		for (int32  I = 0; I < InSolverSettings.Iterations; ++I)
		{
			RunConstraintPass(I, InSolverSettings.Iterations);
		}
	};

	// run the sub iterations with non-chain bodies locked
	RunSubIterations();
	
	// run the main iterations with all bodies un-locked
	RunMainIterations();
}

bool FPBIKSolver::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/FBIK"));
	
	if (bReadyToSimulate)
	{
		return true;
	}

	bReadyToSimulate = false;

	if (!InitBones())
	{
		return false;
	}

	if (!InitBodies())
	{
		return false;
	}

	if (!InitConstraints())
	{
		return false;
	}

	bReadyToSimulate = true;

	return true;
}

bool FPBIKSolver::InitBones()
{
	using PBIK::FEffector;
	using PBIK::FBone;

	if (Bones.IsEmpty())
	{
		UE_LOGF(LogPBIKSolver, Warning, "PBIK: no bones added to solver. Cannot initialize.");
		return false;
	}

	if (Effectors.IsEmpty())
	{
		UE_LOGF(LogPBIKSolver, Warning, "PBIK: no effectors added to solver. Cannot initialize.");
		return false;
	}

	// record solver root pointer
	int32  NumSolverRoots = 0;
	for (FBone& Bone : Bones)
	{
		if (Bone.bIsSolverRoot)
		{
			SolverRoot = &Bone;
			++NumSolverRoots;
		}
	}

	if (!SolverRoot)
	{
		UE_LOGF(LogPBIKSolver, Warning, "PBIK: root bone not set or not found. Cannot initialize.");
		return false;
	}

	if (NumSolverRoots > 1)
	{
		UE_LOGF(LogPBIKSolver, Warning, "PBIK: more than 1 bone was marked as solver root. Cannot initialize.");
		return false;
	}

	// initialize pointers to parents
	for (FBone& Bone : Bones)
	{
		// no parent on root, remains null
		if (Bone.ParentIndex == -1)
		{
			continue;
		}

		// validate parent
		const bool bIndexInRange = Bone.ParentIndex >= 0 && Bone.ParentIndex < Bones.Num();
		if (!bIndexInRange)
		{
			UE_LOGF(LogPBIKSolver, Warning, "PBIK: bone found with invalid parent index. Cannot initialize.");
			return false;
		}

		// record parent
		Bone.Parent = &Bones[Bone.ParentIndex];
	}

	// initialize bIsChildOfSolverRoot
	for (FBone& Bone : Bones)
	{
		if (Bone.bIsSolverRoot)
		{
			Bone.bIsPartOfSolvedBranch = true;
			continue;
		}
		
		FBone* CurrentParent = Bone.Parent;
		while (CurrentParent)
		{
			if (CurrentParent->bIsSolverRoot)
			{
				Bone.bIsPartOfSolvedBranch = true;
				break;
			}
			CurrentParent = CurrentParent->Parent;
		}
	}

	// walk upwards from each effector to the root to initialize "Bone.IsSolved"
	for (const FEffector& Effector : Effectors)
	{
		FBone* CurrentBone = Effector.Bone;
		CurrentBone->bIsSolved = true;
		while (CurrentBone)
		{
			if (CurrentBone->bIsSolverRoot)
			{
				CurrentBone->bIsSolved = true;
				break;
			}
			CurrentBone->bIsSolved = true;
			CurrentBone = CurrentBone->Parent;
		}
	}

	// initialize children lists
	for (FBone& Parent : Bones)
	{
		for (FBone& Child : Bones)
		{
			if (Child.bIsSolved && Child.Parent == &Parent)
			{
				Parent.Children.Add(&Child);
			}
		}
	}

	// initialize IsSubRoot flag
	for (FBone& Bone : Bones)
	{
		Bone.bIsSubRoot = Bone.Children.Num() > 1 || Bone.bIsSolverRoot || Bone.EffectorIndex != INDEX_NONE;
	}

	// store initial local rotation (preferred angles are relative to this)
	for (FBone& Bone : Bones)
	{
		const FQuat LocalRotation = Bone.Parent ? Bone.Parent->Rotation.Inverse() * Bone.Rotation : Bone.Rotation;
		Bone.LocalRotationInitial = LocalRotation.Rotator();
	}

	return true;
}

bool FPBIKSolver::InitBodies()
{
	using PBIK::FEffector;
	using PBIK::FRigidBody;
	using PBIK::FBone;

	Bodies.Empty();
	
	// create bodies
	for (const FEffector& Effector : Effectors)
	{
		FBone* NextBone = Effector.Bone;
		while (true)
		{
			FBone* BodyBone = NextBone->bIsSolverRoot ? NextBone : NextBone->Parent;
			if (!BodyBone)
			{
				UE_LOGF(LogPBIKSolver, Warning, "PBIK: effector is on bone that is not on or below root bone.");
				return false;
			}

			AddBodyForBone(BodyBone);

			NextBone = BodyBone;
			if (NextBone == SolverRoot)
			{
				break;
			}
		}
	}

	// initialize bodies
	for (FRigidBody& Body : Bodies)
	{
		Body.Initialize(SolverRoot);
	}

	// sort bodies leaf to root
	Bodies.Sort();
	Algo::Reverse(Bodies);

	// store pointers to bodies on bones (after sort!)
	for (FRigidBody& Body : Bodies)
	{
		Body.Bone->Body = &Body;
	}

	return true;
}

void FPBIKSolver::AddBodyForBone(PBIK::FBone* Bone)
{
	using PBIK::FRigidBody;
	for (const FRigidBody& Body : Bodies)
	{
		if (Body.Bone == Bone)
		{
			return; // no duplicates
		}
	}
	Bodies.Emplace(Bone);
}

bool FPBIKSolver::InitConstraints()
{
	using PBIK::FEffector;
	using PBIK::FBone;
	using PBIK::FRigidBody;
	using PBIK::FPinConstraint;
	using PBIK::FJointConstraint;

	Constraints.Empty();
	
	// Add Effector Pin Constraints
	// (these are added first since they create the pulling effect which needs resolved by the joint constraints)
	for (FEffector& Effector : Effectors)
	{
		const FBone* BodyBone = Effector.Bone->bIsSolverRoot ? Effector.Bone : Effector.Bone->Parent;
		if (!BodyBone)
		{
			UE_LOGF(LogPBIKSolver, Warning, "PBIK: effector is on bone that does not have a parent.");
			return false;
		}

		FRigidBody* Body = BodyBone->Body;
		TSharedPtr<FPinConstraint> Constraint = MakeShared<FPinConstraint>(Body, Effector.Bone);
		Effector.Pin = Constraint.Get();
		Body->Effector = &Effector;
		Body->Pin = Constraint.Get();
		Constraints.Add(Constraint);
	}

	// Add joint constraints
	// (constrain all bodies together (child to parent))
	for (FRigidBody& Body : Bodies)
	{
		FRigidBody* ParentBody = Body.GetParentBody();
		if (!ParentBody)
		{
			continue; // root
		}

		TSharedPtr<FJointConstraint> Constraint = MakeShared<FJointConstraint>(ParentBody, &Body);
		Constraints.Add(Constraint);
	}

	// now we can set the initial effector depths
	constexpr bool bForceUpdate = true;
	UpdateEffectorDepths(bForceUpdate);
	
	return true;
}

void FPBIKSolver::UpdateEffectorDepths(bool bForceUpdate)
{
	using PBIK::FEffector;
	using PBIK::FBone;
	using PBIK::FRigidBody;
	
	// initialize Effector's nearest ParentSubRoot (FBody) pointer
	// must be done AFTER setting: Bone.IsSubRoot/IsSolverRoot/Parent
	bool bIsEffectorDepthDirty = false;
	bHasSubChains = false;
	for (const FEffector& Effector : Effectors)
	{
		if (bForceUpdate || (Effector.ChainDepthInitializedWith != Effector.Settings.ChainDepth))
		{
			bIsEffectorDepthDirty = true;
		}
		if (Effector.Settings.ChainDepth > 0)
		{
			bHasSubChains = true;
		}
	}

	if (bIsEffectorDepthDirty)
	{
		// reset body states related to sub chains
		for (FRigidBody& Body : Bodies)
		{
			Body.bIsPartOfSubChain = false;
			Body.bIsLockedBySubSolve = false;
		}
		// walk up each effector and set chain states
		for (FEffector& EffectorToUpdate : Effectors)
		{
			EffectorToUpdate.UpdateChainStates();
		}
	}
}

PBIK::FDebugDraw* FPBIKSolver::GetDebugDraw()
{
	DebugDraw.Solver = this;
	return &DebugDraw;
}

void FPBIKSolver::Reset()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/FBIK"));
	
	bReadyToSimulate = false;
	SolverRoot = nullptr;
	Bodies.Reset();
	Bones.Reset();
	Constraints.Reset();
	Effectors.Reset();
}

bool FPBIKSolver::IsReadyToSimulate() const
{
	return bReadyToSimulate;
}

int32 FPBIKSolver::AddBone(
	const FName Name,
	const int32 ParentIndex,
	const FVector& InOrigPosition,
	const FQuat& InOrigRotation,
	bool bIsSolverRoot)
{
	return Bones.Emplace(Name, ParentIndex, InOrigPosition, InOrigRotation, bIsSolverRoot);
}

int32 FPBIKSolver::AddEffector(const FName BoneName)
{
	for (PBIK::FBone& Bone : Bones)
	{
		if (Bone.Name == BoneName)
		{
			const int32 NewEffectorIndex = Effectors.Emplace(&Bone);
			Bone.EffectorIndex = NewEffectorIndex;
			return NewEffectorIndex;
		}
	}

	return -1;
}

void FPBIKSolver::SetBoneTransform(
	const int32 Index,
	const FTransform& InTransform)
{
	check(Index >= 0 && Index < Bones.Num());
	Bones[Index].Position = InTransform.GetLocation();
	Bones[Index].Rotation = InTransform.GetRotation();
	Bones[Index].Scale = InTransform.GetScale3D();
}

PBIK::FBoneSettings* FPBIKSolver::GetBoneSettings(const int32 Index)
{
	// make sure to call Initialize() before applying bone settings
	if (!ensureMsgf(bReadyToSimulate, TEXT("PBIK: trying to access Bone Settings before Solver is initialized.")))
	{
		return nullptr;
	}

	if (!ensureMsgf(Bones.IsValidIndex(Index), TEXT("PBIK: trying to access Bone Settings with invalid bone index.")))
	{
		return nullptr;
	}

	if (!Bones[Index].Body)
	{
		// Bone is not part of the simulation. This happens if the bone is not located between an effector and the
		// root of the solver. Not necessarily an error, as some systems dynamically disable effectors which can leave
		// orphaned Bone Settings, so we simply ignore them.
		return nullptr;
	}

	return &Bones[Index].Body->J;
}

void FPBIKSolver::GetBoneGlobalTransform(const int32 Index, FTransform& OutTransform)
{
	check(Index >= 0 && Index < Bones.Num());
	const PBIK::FBone& Bone = Bones[Index];
	OutTransform.SetLocation(Bone.Position);
	OutTransform.SetRotation(Bone.Rotation);
	OutTransform.SetScale3D(Bone.Scale);
}

int32 FPBIKSolver::GetBoneIndex(FName BoneName) const
{
	return Bones.IndexOfByPredicate([&BoneName](const PBIK::FBone& Bone) { return Bone.Name == BoneName; });
}

void FPBIKSolver::SetEffectorGoal(
	const int32 Index,
	const FVector& InGoalPosition,
	const FQuat& InGoalRotation,
	const PBIK::FEffectorSettings& InSettings)
{
	check(Index >= 0 && Index < Effectors.Num());
	Effectors[Index].SetGoal(InGoalPosition, InGoalRotation, InSettings);
}

// given a set of points in an initial configuration and the same set of points in a deformed configuration,
// this function outputs a quaternion that represents the "best fit" rotation that rotates the initial points to the
// current points.
FQuat FPBIKSolver::GetRotationFromDeformedPoints(
	const TArrayView<FVector>& InInitialPoints,
	const TArrayView<FVector>& InCurrentPoints,
	FVector& OutInitialCentroid,
	FVector& OutCurrentCentroid)
{
	check(InInitialPoints.Num() == InCurrentPoints.Num());

	// must have more than 1 point to generate a gradient
	if (InInitialPoints.Num() <= 1)
	{
		OutInitialCentroid = (InInitialPoints.Num() == 1) ? InInitialPoints[0] : FVector::ZeroVector;
		OutCurrentCentroid = (InCurrentPoints.Num() == 1) ? InCurrentPoints[0] : FVector::ZeroVector;
		return FQuat::Identity;
	}
	
	// calculate initial and current centroids
	OutInitialCentroid = FVector::ZeroVector;
	OutCurrentCentroid = FVector::ZeroVector;
	for (int32 PointIndex=0; PointIndex<InInitialPoints.Num(); ++PointIndex)
	{
		OutInitialCentroid += InInitialPoints[PointIndex];
		OutCurrentCentroid += InCurrentPoints[PointIndex];
	}

	// average centroids
	const float InvNumEffectors = 1.0f / static_cast<float>(InInitialPoints.Num());
	OutInitialCentroid *= InvNumEffectors;
	OutCurrentCentroid *= InvNumEffectors;

	// accumulate deformation gradient to extract a rotation from
	FVector DX = FVector::ZeroVector; // DX, DY, DZ are rows of 3x3 deformation gradient tensor
	FVector DY = FVector::ZeroVector;
	FVector DZ = FVector::ZeroVector;
	for (int32 PointIndex=0; PointIndex<InInitialPoints.Num(); ++PointIndex)
	{
		// accumulate the deformation gradient tensor for all points
		// "Meshless Deformations Based on Shape Matching"
		// Equation 7 describes accumulation of deformation gradient from points
		//
		// P is normalized vector from INITIAL centroid to INITIAL point
		// Q is normalized vector from CURRENT centroid to CURRENT point
		FVector P = (InInitialPoints[PointIndex] - OutInitialCentroid).GetSafeNormal();
		FVector Q = (InCurrentPoints[PointIndex] - OutCurrentCentroid).GetSafeNormal();
		// PQ^T is the outer product of P and Q which is a 3x3 matrix
		// https://en.m.wikipedia.org/wiki/Outer_product
		DX += FVector(P[0]*Q[0], P[0]*Q[1], P[0]*Q[2]);
		DY += FVector(P[1]*Q[0], P[1]*Q[1], P[1]*Q[2]);
		DZ += FVector(P[2]*Q[0], P[2]*Q[1], P[2]*Q[2]);
	}

	// extract "best fit" rotation from deformation gradient
	FQuat Q = FQuat::Identity;
	constexpr int32 MaxIter = 50;
	// "A Robust Method to Extract the Rotational Part of Deformations" equation 7
	// https://matthias-research.github.io/pages/publications/stablePolarDecomp.pdf
	for (unsigned int Iter = 0; Iter < MaxIter; Iter++)
	{
		FMatrix R = FRotationMatrix::Make(Q);
		FVector RCol0(R.M[0][0], R.M[0][1], R.M[0][2]);
		FVector RCol1(R.M[1][0], R.M[1][1], R.M[1][2]);
		FVector RCol2(R.M[2][0], R.M[2][1], R.M[2][2]);
		FVector Omega = RCol0.Cross(DX) + RCol1.Cross(DY) + RCol2.Cross(DZ);
		Omega *= 1.0f / (fabs(RCol0.Dot(DX) + RCol1.Dot(DY) + RCol2.Dot(DZ)) + SMALL_NUMBER);
		const float W = Omega.Size();
		if (W < SMALL_NUMBER)
		{
			break;
		}
		Q = FQuat(FQuat((1.0 / W) * Omega, W)) * Q;
		Q.Normalize();
	}

	return Q;
}

