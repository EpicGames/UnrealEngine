// Copyright Epic Games, Inc. All Rights Reserved.

#include "PBIK.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FPBIKModule"

DEFINE_LOG_CATEGORY(LogPBIKSolver);

void FPBIKModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
}

void FPBIKModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPBIKModule, PBIK)
