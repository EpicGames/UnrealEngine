// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_PBIK.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Units/RigUnitContext.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyElements.h"
#include "Misc/HashBuilder.h"
#include "PBIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_PBIK)

FRigUnit_PBIK_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	LLM_SCOPE_BYNAME(TEXT("Animation/FBIK"));

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}
	
	// check if any inputs have been modified that would require reinitialization
	FHashBuilder HashBuilder;
	for (const FPBIKEffector& Effector : Effectors)
	{
		HashBuilder << Effector.Bone;
	}
	for (const FPBIKBoneSetting& BoneSetting : BoneSettings)
	{
		HashBuilder << BoneSetting.Bone;
	}
	uint32 Hash = HashBuilder.GetHash();
	WorkData.bNeedsInit = WorkData.bNeedsInit ? true : Hash != WorkData.HashInitializedWith;

	// re-init if required
	if (WorkData.bNeedsInit)
	{
		WorkData.BoneSettingToSolverBoneIndex.Reset();
		WorkData.Solver.Reset();
		WorkData.SolverBoneToElementIndex.Reset();
		EffectorSolverIndices.Reset();
		
		// check how many effectors are assigned to a bone
		int NumEffectors = 0;
		for (const FPBIKEffector& Effector : Effectors)
		{
			if (Hierarchy->GetIndex(FRigElementKey(Effector.Bone, ERigElementType::Bone)) != INDEX_NONE)
			{
				++NumEffectors; // bone is set and exists!
			}
		}

		// validate inputs are ready to be initialized
		bool bHasEffectors = NumEffectors > 0;
		bool bRootIsAssigned = Root != NAME_None;
		if (!(bHasEffectors && bRootIsAssigned))
		{
			return; // not setup yet
		}
		
		// create solver bone index to elements index map
		TArray<FRigBoneElement*> BoneElements = Hierarchy->GetBones(true);
		for (int B = 0; B < BoneElements.Num(); ++B)
		{
			WorkData.SolverBoneToElementIndex.Add(BoneElements[B]->GetIndex());
		}

		// create bones
		for (int B = 0; B < BoneElements.Num(); ++B)
		{
			FName Name = BoneElements[B]->GetFName();

			// get the first parent that is not excluded
			int32 ParentElementIndex = INDEX_NONE;
			FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(BoneElements[B]);
			while (ParentElement)
			{
				if (!ExcludedBones.Contains(ParentElement->GetKey().Name))
				{
					ParentElementIndex = ParentElement->GetIndex();
					break;
				}
				ParentElement = Hierarchy->GetFirstParent(ParentElement);
			}

			// get the parent bone solver index
			int ParentIndex = -1;
			for (int P=0; P<WorkData.SolverBoneToElementIndex.Num(); ++P)
			{
				if (WorkData.SolverBoneToElementIndex[P] == ParentElementIndex)
				{
					ParentIndex = P;
					break;
				}
			}
			
			const FTransform OrigTransform = Hierarchy->GetTransform(BoneElements[B], ERigTransformType::InitialGlobal);
			const FVector InOrigPosition = OrigTransform.GetLocation();
			const FQuat InOrigRotation = OrigTransform.GetRotation();
			bool bIsRoot = BoneElements[B]->GetFName() == Root;
			WorkData.Solver.AddBone(Name, ParentIndex, InOrigPosition, InOrigRotation, bIsRoot);
		}
		
		// create effectors
		for (const FPBIKEffector& Effector : Effectors)
		{
			int32 IndexInSolver = WorkData.Solver.AddEffector(Effector.Bone);
			EffectorSolverIndices.Add(IndexInSolver);
		}
		
		// initialize
		if (!WorkData.Solver.Initialize())
		{
			UE_LOGF(LogPBIKSolver, Warning, "PBIK: Solver failed to initialize in '%ls'", *GetPathNameSafe(Hierarchy));
		}
		WorkData.bNeedsInit = false;
		WorkData.HashInitializedWith = Hash;
	}

	if (!WorkData.Solver.IsReadyToSimulate())
	{
		return;
	}

	if (EffectorSolverIndices.Num() != Effectors.Num())
	{
		return;
	}

	// set bones to input pose
	for(int32 BoneIndex = 0; BoneIndex < WorkData.Solver.GetNumBones(); BoneIndex++)
	{
		const FTransform GlobalTransform = Hierarchy->GetGlobalTransform(WorkData.SolverBoneToElementIndex[BoneIndex]);
		WorkData.Solver.SetBoneTransform(BoneIndex, GlobalTransform);
	}

	// invalidate the name lookup for the settings
	if(WorkData.BoneSettingToSolverBoneIndex.Num() != BoneSettings.Num())
	{
		WorkData.BoneSettingToSolverBoneIndex.Reset();
		while(WorkData.BoneSettingToSolverBoneIndex.Num() < BoneSettings.Num())
		{
			WorkData.BoneSettingToSolverBoneIndex.Add(INDEX_NONE);
		}
	}

	// update bone settings
	for (int32 BoneSettingIndex = 0; BoneSettingIndex < BoneSettings.Num(); BoneSettingIndex++)
	{
		const FPBIKBoneSetting& BoneSetting = BoneSettings[BoneSettingIndex];

		if(WorkData.BoneSettingToSolverBoneIndex[BoneSettingIndex] == INDEX_NONE)
		{
			WorkData.BoneSettingToSolverBoneIndex[BoneSettingIndex] = WorkData.Solver.GetBoneIndex(BoneSetting.Bone);
			if(WorkData.BoneSettingToSolverBoneIndex[BoneSettingIndex] == INDEX_NONE)
			{
				continue;
			}
		}

		int32 BoneIndex = WorkData.BoneSettingToSolverBoneIndex[BoneSettingIndex];
		if (PBIK::FBoneSettings* InternalSettings = WorkData.Solver.GetBoneSettings(BoneIndex))
		{
			BoneSetting.CopyToCoreStruct(*InternalSettings);
		}
	}

	// update effectors
	for (int E = 0; E < Effectors.Num(); ++E)
	{
		if (EffectorSolverIndices[E] == -1)
		{
			continue;
		}

		const FPBIKEffector& Effector = Effectors[E];
		FVector Position = Effector.Transform.GetLocation();
		FQuat Rotation = Effector.Transform.GetRotation();

		PBIK::FEffectorSettings EffectorSettings;
		EffectorSettings.PositionAlpha = Effector.PositionAlpha;
		EffectorSettings.RotationAlpha = Effector.RotationAlpha;
		EffectorSettings.StrengthAlpha = Effector.StrengthAlpha;
		EffectorSettings.ChainDepth = Effector.ChainDepth;
		EffectorSettings.PullChainAlpha = Effector.PullChainAlpha;
		EffectorSettings.PinRotation = Effector.PinRotation;
		
		WorkData.Solver.SetEffectorGoal(EffectorSolverIndices[E], Position, Rotation, EffectorSettings);
	}

	// solve
	WorkData.Solver.Solve(Settings);

	// copy transforms back
	for(int32 BoneIndex = 0; BoneIndex < WorkData.Solver.GetNumBones(); BoneIndex++)
	{
		FTransform NewTransform;
		WorkData.Solver.GetBoneGlobalTransform(BoneIndex, NewTransform);
		constexpr bool bPropagateTransform = false;
		Hierarchy->SetGlobalTransform(WorkData.SolverBoneToElementIndex[BoneIndex], NewTransform, false, bPropagateTransform);
	}
}

