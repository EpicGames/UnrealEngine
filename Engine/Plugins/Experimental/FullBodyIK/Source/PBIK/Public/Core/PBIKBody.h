// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"

struct FPBIKSolverSettings;

namespace PBIK
{
	struct FPinConstraint;
	struct FJointConstraint;
	struct FRigidBody;
	struct FEffector;

struct FBone
{
	FName Name;
	int32 ParentIndex = -2; // -2 is unset, -1 for root, or 0...n otherwise
	FVector Position;
	FQuat Rotation;
	FVector Scale; // just passed through, not modified
	FVector LocalPositionFromInput;
	FQuat LocalRotationFromInput;
	FRotator LocalRotationInitial;

	// initialized - these fields are null/empty until after Solver::Initialize()
	FRigidBody* Body = nullptr;
	FBone* Parent = nullptr;
	TArray<FBone*> Children;
	int32 EffectorIndex = INDEX_NONE;
	bool bIsSolverRoot = false;
	bool bIsSolved = false;
	bool bIsSubRoot = false;
	bool bIsPartOfSolvedBranch = false;
	float Length = 0.f;
	// initialized

	FBone(
		const FName InName,
		const int32& InParentIndex,		// must pass -1 for root of whole skeleton
		const FVector& InOrigPosition,
		const FQuat& InOrigRotation,
		bool bInIsSolverRoot);

	bool HasChild(const FBone* Bone);
	void UpdateFromInputs();
};

enum class ELimitType : uint8
{
	Free,
	Limited,
	Locked,
};

struct FBoneSettings
{
	float RotationStiffness = 0.0f; // range (0, 1)
	float PositionStiffness = 0.0f; // range (0, 1)

	ELimitType X;
	float MinX = 0.0f; // range (-180, 180)
	float MaxX = 0.0f;

	ELimitType Y;
	float MinY = 0.0f;
	float MaxY = 0.0f;

	ELimitType Z;
	float MinZ = 0.0f;
	float MaxZ = 0.0f;

	bool bUsePreferredAngles = false;
	FRotator PreferredAngles = FRotator::ZeroRotator;
};

struct FRigidBody
{
	FBone* Bone = nullptr;
	FBoneSettings J;
	FPinConstraint* Pin = nullptr;
	FEffector* Effector = nullptr;
	const FPBIKSolverSettings* SolverSettings;

	FVector Position;
	FQuat Rotation;
	FVector InputPosition;
	FQuat InitialRotation;
	FVector BoneLocalPosition;

	float InvMass = 0.f;
	float Mass = 0.f;

	float MaxAngle = 0.0f;

	bool bIsLockedBySubSolve = false;
	bool bIsPartOfSubChain = false;
	
private:

	int32 NumBonesToRoot = 0;

public:

	FRigidBody(FBone* InBone);

	void Initialize(const FBone* SolverRoot);

	void UpdateFromInputs(const FPBIKSolverSettings& Settings);

	void UpdateTransformAndMassFromBones();

	int GetNumBonesToRoot() const;

	FRigidBody* GetParentBody() const;

	bool IsPositionLocked() const;

	bool IsRotationLocked() const;

	void ApplyPushToRotateBody(const FVector& Push, const FVector& Offset);
	
	void ApplyPositionDelta(const FVector& InDeltaP);

	void ApplyRotationDelta(const FQuat& InDelta);
};

// for sorting Bodies hierarchically (root to leaf order)
inline bool operator<(const FRigidBody& Lhs, const FRigidBody& Rhs)
{ 
	return Lhs.GetNumBonesToRoot() < Rhs.GetNumBonesToRoot();
}

} // namespace
