// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"

struct FPBIKSolverSettings;

namespace PBIK
{
	struct FRigidBody;
	struct FBone;

struct FConstraint
{
	bool bEnabled = true;

	virtual void Solve(const FPBIKSolverSettings& Settings) = 0;
	virtual void RemoveStretch(const float Percent){};
	virtual void FinalPass(){};
	virtual void UpdateFromInputs(){};
};

struct FJointConstraint : public FConstraint
{

private:

	FRigidBody* A;
	FRigidBody* B;

	FVector PinPointLocalToA;
	FVector PinPointLocalToB;

	FVector XOrig;
	FVector YOrig;
	FVector ZOrig;

	FVector XA;
	FVector YA;
	FVector ZA;
	FVector XB;
	FVector YB;
	FVector ZB;

	FVector ZBProjOnX;
	FVector ZBProjOnY;
	FVector YBProjOnZ;

	float AngleX;
	float AngleY;
	float AngleZ;

	bool bInFinalPass = false;

public:

	FJointConstraint(FRigidBody* InA, FRigidBody* InB);

	virtual ~FJointConstraint() = default;

	virtual void Solve(const FPBIKSolverSettings& Settings) override;

	virtual void RemoveStretch(const float Percent) override;

	virtual void FinalPass() override;

	virtual void UpdateFromInputs() override;

private:

	FVector GetPositionCorrection(FVector& OutBodyToA, FVector& OutBodyToB) const;

	void ApplyRotationCorrection(FQuat DeltaQ) const;

	void UpdateJointLimits();

	void RotateWithinLimits(
		float MinAngle,
		float MaxAngle,
		float CurrentAngle,
		FVector RotAxis,
		FVector CurVec,
		FVector RefVec) const;

	void RotateToAlignAxes(const FVector& AxisA, const FVector& AxisB) const;

	void UpdateLocalRotateAxes(bool bX, bool bY, bool bZ);

	void DecomposeRotationAngles();

	float SignedAngleBetweenNormals(
		const FVector& From,
		const FVector& To,
		const FVector& Axis) const;
};

struct FPinConstraint : public FConstraint
{
private:
	
	FVector GoalPosition;
	FQuat GoalRotation;
	float Alpha = 1.0;
	
	FRigidBody* Body;
	FBone* PinBone;
	FVector PinPointLocalToA;

public:
	
	FPinConstraint(FRigidBody* InBody, FBone* InPinBone);

	virtual ~FPinConstraint() = default;

	virtual void Solve(const FPBIKSolverSettings& Settings) override;

	virtual void UpdateFromInputs() override;

	void SetGoal(const FVector& InGoalPosition, const FQuat& InGoalRotation, const float InAlpha);

	FVector GetCurrentPinPoint() const;

private:

	FVector GetPositionCorrection(FVector& OutBodyToPinPoint) const;

	friend FRigidBody;
	friend FJointConstraint;
};

} // namespace
