// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

struct FPBIKSolver;

namespace PBIK
{
	struct FDebugLine
	{
		FVector A;
		FVector B;

		FDebugLine(const FVector& InA, const FVector& InB)
		{
			A = InA;
			B = InB;
		}
	};

	struct FDebugDraw
	{
		FPBIKSolver* Solver;

		void GetDebugLinesForBodies(TArray<FDebugLine>& OutLines);
	};
}
