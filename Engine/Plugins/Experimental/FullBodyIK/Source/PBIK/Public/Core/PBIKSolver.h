// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PBIKDebug.h"

#include "PBIKSolver.generated.h"

namespace PBIK { struct FBone; }
namespace PBIK { struct FBoneSettings; }
namespace PBIK { struct FConstraint; }
namespace PBIK { struct FPinConstraint; }

DECLARE_CYCLE_STAT(TEXT("PBIK Solve"), STAT_PBIK_Solve, STATGROUP_Anim);

namespace PBIK
{

static float GLOBAL_UNITS = 100.0f; // (1.0f = meters), (100.0f = centimeters)
static float MIN_MASS = 0.5f;

// A long tail ease out function. Input range, 0-1. 
inline static float QuarticEaseOut(const float& Input){ return (FMath::Pow(Input-1.0f, 4.0f) * -1.0f) + 1.0f; };
// A C2 continuous ease out function. Input range, 0-1
inline static float CircularEaseOut(const float& Input){ return FMath::Sqrt(1.0f - FMath::Pow(Input - 1.0f, 2.0f)); };
// An ease out function. Input range, 0-1.
inline static float SquaredEaseOut(const float& Input){ return (FMath::Pow(Input-1.0f, 2.0f) * -1.0f) + 1.0f; };

struct FRigidBody;

struct FEffectorSettings
{
	/** Range 0-1, default is 1. Blend between the input bone position (0.0) and the current effector position (1.0).*/
	float PositionAlpha = 1.0f;

	/** Range 0-1, default is 1. Blend between the input bone rotation (0.0) and the current effector rotation (1.0).*/
	float RotationAlpha = 1.0f;

	/** Range 0-1 (default is 1.0). The strength of the effector when pulling the bone towards it's target location.
	* At 0.0, the effector does not pull at all, but the bones between the effector and the root will still slightly resist motion from other effectors.
	* This can thus act as a "stabilizer" of sorts for parts of the body that you do not want to behave in a pure FK fashion.
	*/
	float StrengthAlpha = 1.0f;

	/** Range 0-inf (default is 0). Explicitly set the number of bones up the hierarchy to consider part of this effector's 'chain'.
	* The "chain" of bones is used to apply Preferred Angles, Pull Chain Alpha and Chain "Sub Solves".
	* If left at 0, the solver will attempt to determine the root of the chain by searching up the hierarchy until it finds a branch or another effector, whichever it finds first.
	*/
	int32 ChainDepth = 0;

	/** Range 0-1 (default is 1.0). When enabled (greater than 0.0), the solver internally partitions the skeleton into 'chains' which extend from the effector to the nearest fork in the skeleton.
	*These chains are pre-rotated and translated, as a whole, towards the effector targets.
	*This can improve the results for sparse bone chains, and significantly improve convergence on dense bone chains.
	*But it may cause undesirable results in highly constrained bone chains (like robot arms).
	*/
	float PullChainAlpha = 1.0f;

	/** Range 0-1 (default is 1.0).
	*Blends the effector bone rotation between the rotation of the effector transform (1.0) and the rotation of the input bone (0.0).*/
	float PinRotation = 1.0f;
};

struct FEffector
{
	FVector PositionFinal;
	FQuat RotationFinal;

	FVector PositionOrig;
	FQuat RotationOrig;

	FVector PositionGoal;
	FQuat RotationGoal;

	FEffectorSettings Settings;

	FBone* Bone;
	FPinConstraint* Pin;
	FRigidBody* ChainRootBody = nullptr;
	int32 ChainDepthInitializedWith = -1;
	float DistToChainRootInInputPose;
	
	TArray<float> DistancesFromEffector;
	float DistToChainRootAlongBones;

	FEffector(FBone* InBone);

	void SetGoal(
		const FVector& InPositionGoal,
		const FQuat& InRotationGoal,
		const FEffectorSettings& InSettings);

	void UpdateChainStates();
	void UpdateFromInputs(const FBone& InSolverRoot, const bool bAllowStretch);
	float CalculateDistanceToChainRoot() const;
	float CalculateDistanceToSolveRootAlongBones() const;
	void ApplyPreferredAngles() const;
};
	
} // namespace

UENUM(BlueprintType)
enum class EPBIKRootBehavior : uint8
{
	PrePull,
	PinToInput,
	Free,
};

USTRUCT(BlueprintType)
struct FRootPrePullSettings
{
	GENERATED_BODY()
	
	/** Range 0-1, default is 0. Apply a large scale rotation offset to the entire body prior to constraint solving.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootRotation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlpha = 0.0f;

	/** Range 0-1, default is 1. Blend contribution to rotation offset in the X axis in component space.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootRotation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlphaX = 1.0f;

	/** Range 0-1, default is 1. Blend contribution to rotation offset in the Y axis in component space.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootRotation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlphaY = 1.0f;

	/** Range 0-1, default is 1. Blend contribution to rotation offset in the Z axis in component space.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootRotation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlphaZ = 1.0f;

	/** Range 0-1, default is 1. Apply a large scale position offset to the entire body prior to constraint solving.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootPosition, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlpha = 1.0f;

	/** Range 0-1, default is 1. Blend contribution to position offset in the X axis in component space.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootPosition, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlphaX = 1.0f;

	/** Range 0-1, default is 1. Blend contribution to position offset in the Y axis in component space.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootPosition, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlphaY = 1.0f;

	/** Range 0-1, default is 1. Blend contribution to position offset in the Z axis in component space.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootPosition, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlphaZ = 1.0f;
};

USTRUCT(BlueprintType)
struct FPBIKSolverSettings
{
	GENERATED_BODY()

	/** High iteration counts can help solve complex joint configurations with competing constraints, but will increase runtime cost. Default is 20. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0", ClampMax = "1000", UIMin = "0.0", UIMax = "200.0"))
	int32 Iterations = 20;

	/** Iterations used for sub-chains defined by the Chain Depth of the effectors. These are solved BEFORE the main iteration pass. Default is 0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0", ClampMax = "1000", UIMin = "0.0", UIMax = "200.0"))
	int32 SubIterations = 0;

	/** A global mass multiplier; higher values will make the joints more stiff, but require more iterations. Typical range is 0.0 to 10.0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0", UIMin = "0.0", UIMax = "10.0"))
	float MassMultiplier = 1.0f;

	/** If true, joints will translate to reach the effectors; causing bones to lengthen if necessary. Good for cartoon effects. Default is false. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings)
	bool bAllowStretch = false;

	/** (Default is PrePull) Set the behavior of the solver root.
	*Pre Pull: translates and rotates the root (and all children) by the average motion of the stretched effectors to help achieve faster convergence when reaching far.
	*Pin to Input: locks the translation and rotation of the root bone to the input pose. Overrides any bone settings applied to the root. Good for partial-body solves.
	*Free: treats the root bone like any other and allows it to move freely or according to any bone settings applied to it. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootBehavior)
	EPBIKRootBehavior RootBehavior = EPBIKRootBehavior::PrePull;

	/** Settings only applicable when Root Behavior is set to "PrePull". Use these values to adjust the gross movement and orientation of the entire skeleton. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootBehavior)
	FRootPrePullSettings PrePullRootSettings;

	/** A global multiplier for all Pull Chain Alpha values on all effectors. Range is 0.0 to 1.0. Default is 1.0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AdvancedSettings, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float GlobalPullChainAlpha = 1.0f;

	/** Maximum angle that a joint can be rotated per constraint iteration. Lower this value if the solve is diverging. Range is 0.0 to 180.0. Default is 30. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AdvancedSettings, meta = (ClampMin = "0", ClampMax = "45", UIMin = "0.0", UIMax = "180.0"))
	float MaxAngle = 30.f;

	/** Pushes constraints beyond their normal amount to speed up convergence. Increasing this may speed up convergence, but at the cost of stability. Range is 1.0 - 2.0. Default is 1.3. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AdvancedSettings, meta = (ClampMin = "1", UIMin = "1.0", UIMax = "2.0"))
	float OverRelaxation = 1.3f;
	
	/** DEPRECATED: When true, the solver is reset each tick to start from the current input pose. Default is true.
	 * If false, incoming animated poses are ignored and the solver starts from the results of the previous solve.
	 * In very limited circumstances, it can be beneficial to use the pose from the previous frame.*/
	UPROPERTY()
	bool bStartSolveFromInputPose_DEPRECATED = true;
};

USTRUCT()
struct FPBIKSolver
{
	GENERATED_BODY()

public:

	FPBIKSolver& operator=(const FPBIKSolver&)
	{
		bReadyToSimulate = false;
		return *this;
	}

	PBIK_API PBIK::FDebugDraw* GetDebugDraw();

	//
	// main runtime functions
	//

	PBIK_API bool Initialize();

	PBIK_API void Solve(const FPBIKSolverSettings& InSolverSettings);

	PBIK_API void Reset();

	PBIK_API bool IsReadyToSimulate() const;

	//
	// set input / get output at runtime
	//

	PBIK_API void SetBoneTransform(const int32 Index, const FTransform& InTransform);

	PBIK_API PBIK::FBoneSettings* GetBoneSettings(const int32 Index);

	PBIK_API void SetEffectorGoal(
		const int32 Index, 
		const FVector& InGoalPosition, 
		const FQuat& InGoalRotation, 
		const PBIK::FEffectorSettings& Settings);

	PBIK_API void GetBoneGlobalTransform(const int32 Index, FTransform& OutTransform);

	int32 GetNumBones() const { return Bones.Num(); }

	PBIK_API int32 GetBoneIndex(FName BoneName) const;

	//
	// pre-init /  setup functions
	//

	PBIK_API int32 AddBone(
		const FName Name,
		const int32 ParentIndex,
		const FVector& InOrigPosition,
		const FQuat& InOrigRotation,
		bool bIsSolverRoot);

	PBIK_API int32 AddEffector(const FName BoneName);
	
private:

	bool InitBones();

	bool InitBodies();

	bool InitConstraints();

	void UpdateEffectorDepths(bool bForceUpdate);

	void AddBodyForBone(PBIK::FBone* Bone);

	void UpdateBodies(const FPBIKSolverSettings& Settings);

	void UpdateBonesFromBodies();

	void SolveConstraints(const FPBIKSolverSettings& InSolverSettings);
	
	void ApplyRootPrePull(const EPBIKRootBehavior RootBehavior, const FRootPrePullSettings& PrePullSettings);
	
	void ApplyPullChainAlpha(const float GlobalPullChainAlpha);

	void ApplyPreferredAngles();

private:

	// given a set of points in an initial configuration and the same set of points in a deformed configuration,
	// this function outputs a quaternion that represents the "best fit" rotation that rotates the initial points to the
	// current points.
	static FQuat GetRotationFromDeformedPoints(
		const TArrayView<FVector>& InInitialPoints,
		const TArrayView<FVector>& InCurrentPoints,
		FVector& OutInitialCentroid,
		FVector& OutCurrentCentroid);
	
	PBIK::FBone* SolverRoot = nullptr;
	TArray<PBIK::FBone> Bones;
	TArray<PBIK::FRigidBody> Bodies;
	TArray<TSharedPtr<PBIK::FConstraint>> Constraints;
	TArray<PBIK::FEffector> Effectors;
	bool bHasSubChains = false;
	bool bReadyToSimulate = false;
	
	PBIK::FDebugDraw DebugDraw;
	friend PBIK::FDebugDraw;
};
