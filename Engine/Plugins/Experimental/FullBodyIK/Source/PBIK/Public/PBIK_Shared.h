// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "Core/PBIKBody.h"
#include "PBIK_Shared.generated.h"

UENUM(BlueprintType)
enum class EPBIKLimitType : uint8
{
	Free,
	Limited,
	Locked,
};

USTRUCT(BlueprintType)
struct FPBIKBoneSetting
{
	GENERATED_BODY()

	FPBIKBoneSetting()
		: Bone(NAME_None), 
		X(EPBIKLimitType::Free),
		Y(EPBIKLimitType::Free),
		Z(EPBIKLimitType::Free),
		PreferredAngles(FVector::ZeroVector){}

	/** The Bone that these settings will be applied to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Bone, meta = (CustomWidget = "BoneName"))
	FName Bone;

	/** Range is 0 to 1 (Default is 0). At higher values, the bone will resist rotating (forcing other bones to compensate). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Stiffness, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float RotationStiffness = 0.0f;

	/** Range is 0 to 1 (Default is 0). At higher values, the bone will resist translational motion (forcing other bones to compensate). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Stiffness, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float PositionStiffness = 0.0f;

	/** Limit the rotation angle of the bone on the X axis.
	 *Free: can rotate freely in this axis.
	 *Limited: rotation is clamped between the min/max angles relative to the Skeletal Mesh reference pose.
	 *Locked: no rotation is allowed in this axis (will remain at reference pose angle). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	EPBIKLimitType X;
	/**Range is -180 to 0 (Default is 0). Degrees of rotation in the negative X direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "-180", ClampMax = "0", UIMin = "-180.0", UIMax = "0.0"))
	float MinX = 0.0f;
	/**Range is 0 to 180 (Default is 0). Degrees of rotation in the positive X direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "0", ClampMax = "180", UIMin = "0.0", UIMax = "180.0"))
	float MaxX = 0.0f;

	/** Limit the rotation angle of the bone on the Y axis.
	*Free: can rotate freely in this axis.
	*Limited: rotation is clamped between the min/max angles relative to the Skeletal Mesh reference pose.
	*Locked: no rotation is allowed in this axis (will remain at input pose angle). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	EPBIKLimitType Y;
	/**Range is -180 to 0 (Default is 0). Degrees of rotation in the negative Y direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "-180", ClampMax = "0", UIMin = "-180.0", UIMax = "0.0"))
	float MinY = 0.0f;
	/**Range is 0 to 180 (Default is 0). Degrees of rotation in the positive Y direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "0", ClampMax = "180", UIMin = "0.0", UIMax = "180.0"))
	float MaxY = 0.0f;

	/** Limit the rotation angle of the bone on the Z axis.
	*Free: can rotate freely in this axis.
	*Limited: rotation is clamped between the min/max angles relative to the Skeletal Mesh reference pose.
	*Locked: no rotation is allowed in this axis (will remain at input pose angle). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	EPBIKLimitType Z;
	/**Range is -180 to 0 (Default is 0). Degrees of rotation in the negative Z direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "-180", ClampMax = "0", UIMin = "-180.0", UIMax = "0.0"))
	float MinZ = 0.0f;
	/**Range is 0 to 180 (Default is 0). Degrees of rotation in the positive Z direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "0", ClampMax = "180", UIMin = "0.0", UIMax = "180.0"))
	float MaxZ = 0.0f;

	/**When true, this bone will "prefer" to rotate in the direction specified by the Preferred Angles when the chain it belongs to is compressed.
	 * Preferred Angles should be the first method used to fix bones that bend in the wrong direction (rather than limits).
	 * The resulting angles can be visualized on their own by temporarily setting the Solver iterations to 0 and moving the effectors.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PreferredAngles)
	bool bUsePreferredAngles = false;
	/**The local Euler angles (in degrees) used to rotate this bone when the chain it belongs to is squashed.
	 * This happens by moving the effector at the tip of the chain towards the root of the chain.
	 * This can be used to coerce knees and elbows to bend in the anatomically "correct" direction without resorting to limits (which may require more iterations to converge).*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PreferredAngles)
	FVector PreferredAngles;

	void CopyToCoreStruct(PBIK::FBoneSettings& Settings) const
	{
		Settings.RotationStiffness = RotationStiffness;
		Settings.PositionStiffness = PositionStiffness;
		Settings.X = static_cast<PBIK::ELimitType>(X);
		Settings.MinX = MinX;
		Settings.MaxX = MaxX;
		Settings.Y = static_cast<PBIK::ELimitType>(Y);
		Settings.MinY = MinY;
		Settings.MaxY = MaxY;
		Settings.Z = static_cast<PBIK::ELimitType>(Z);
		Settings.MinZ = MinZ;
		Settings.MaxZ = MaxZ;
		Settings.bUsePreferredAngles = bUsePreferredAngles;
		Settings.PreferredAngles.Pitch = PreferredAngles.Y;
		Settings.PreferredAngles.Yaw = PreferredAngles.Z;
		Settings.PreferredAngles.Roll = PreferredAngles.X;
	}
};
