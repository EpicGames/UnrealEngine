// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMFunction.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"

#include "Core/PBIKSolver.h"
#include "Core/PBIKDebug.h"

#include "PBIK_Shared.h"

#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigUnit_PBIK.generated.h"

#define UE_API PBIK_API

struct FControlRigExecuteContext;

using PBIK::FDebugLine;

USTRUCT(BlueprintType)
struct FPBIKDebug
{
	GENERATED_BODY()

	/** The scale of the debug drawing. */
	UPROPERTY(EditAnywhere, Category = Debug)
	float DrawScale = 1.0f;

	/** If true, turns on debug drawing for the node. */
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawDebug = false;

	void Draw(FRigVMDrawInterface* DrawInterface, FPBIKSolver* Solver) const
	{
		if (!(DrawInterface && Solver && bDrawDebug))
		{
			return;
		}

		const FLinearColor Bright = FLinearColor(0.f, 1.f, 1.f, 1.f);
		DrawInterface->DrawBox(FTransform::Identity, FTransform(FQuat::Identity, FVector(0, 0, 0), FVector(1.f, 1.f, 1.f) * DrawScale * 0.1f), Bright);
	}
};

USTRUCT(BlueprintType)
struct FPBIKEffector
{
	GENERATED_BODY()

	FPBIKEffector()	:
		Bone(NAME_None),
		PositionAlpha(1.0f),
		RotationAlpha(1.0f),
		StrengthAlpha(1.0f),
		ChainDepth(0),
		PullChainAlpha(1.0f),
		PinRotation(1.0f){}

	/** The bone that this effector will pull on. */
	UPROPERTY(EditAnywhere, Category="Effector", meta = (CustomWidget = "BoneName"))
	FName Bone;

	/** The target location and rotation for this effector. The solver will try to get the specified bone to reach this location.*/
	UPROPERTY(EditAnywhere, Category="Effector")
	FTransform Transform;

	/** Range 0-1, default is 1. Blend between the input bone position (0.0) and the current effector position (1.0).*/
	UPROPERTY(EditAnywhere, Category = "Goal Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlpha = 1.0f;

	/** Range 0-1, default is 1. Blend between the input bone rotation (0.0) and the current effector rotation (1.0).*/
	UPROPERTY(EditAnywhere, Category = "Goal Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlpha = 1.0f;

	/** Range 0-1 (default is 1.0). The strength of the effector when pulling the bone towards it's target location.
	 * At 0.0, the effector does not pull at all, but the bones between the effector and the root will still slightly resist motion from other effectors.
	 * This can thus act as a "stabilizer" of sorts for parts of the body that you do not want to behave in a pure FK fashion.
	 */
	UPROPERTY(EditAnywhere, Category="Effector", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float StrengthAlpha = 1.0f;

	/** Range 0-inf (default is 0). Explicitly set the number of bones up the hierarchy to consider part of this effector's 'chain'.
	* The "chain" of bones is used to apply Preferred Angles, Pull Chain Alpha and "Sub Solve" iterations.
	* If left at 0, the solver will attempt to determine the root of the chain by searching up the hierarchy until it finds a branch or another effector, whichever it finds first.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Full Body IK Effector", meta = (ClampMin = "0", UIMin = "0"))
	int32 ChainDepth = 0;

	/** Range 0-1 (default is 1.0). When enabled (greater than 0.0), the solver internally partitions the skeleton into 'chains' which extend from the effector to the nearest fork in the skeleton.
	 *These chains are pre-rotated and translated, as a whole, towards the effector targets.
	 *This can improve the results for sparse bone chains, and significantly improve convergence on dense bone chains.
	 *But it may cause undesirable results in highly constrained bone chains (like robot arms).
	 */
	UPROPERTY(EditAnywhere, Category="Effector", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PullChainAlpha = 0.0f;

	/** Range 0-1 (default is 1.0).
	 *Blends the effector bone rotation between the rotation of the effector transform (1.0) and the rotation of the input bone (0.0).*/
	UPROPERTY(EditAnywhere, Category="Effector", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PinRotation = 1.0f;
};

USTRUCT()
struct FPBIKWorkData
{
	GENERATED_BODY()
	
	FPBIKWorkData() :
		bNeedsInit(true),
		HashInitializedWith(-1)
	{
	}
	
	UPROPERTY(transient)
	bool bNeedsInit;

	UPROPERTY(transient)
	uint32 HashInitializedWith;
	
	UPROPERTY(transient)
	TArray<int32> BoneSettingToSolverBoneIndex;

	UPROPERTY(transient)
	TArray<int32> SolverBoneToElementIndex;
	
	UPROPERTY(transient)
	FPBIKSolver Solver;
};

/*
 * Based on a Position Based solver at core, this node can solve multi chains within a root using multi effectors
 */
USTRUCT(meta=(DisplayName="Full Body IK", NodeColor="0 1 1"))
struct FRigUnit_PBIK : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_PBIK() : Root(NAME_None) { }

	/**This is usually the top-most skinned bone; often the "Pelvis" or "Hips", but can be set to any bone.
	 *Bones above the root will be ignored by the solver.
	 *Bones that are located *between* the Root and the effectors will be included in the solve.*/
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	FName Root;

	/** An array of effectors. These specify target transforms for different parts of the skeleton. */
	UPROPERTY(meta = (Input))
	TArray<FPBIKEffector> Effectors;
	
	UPROPERTY(transient)
	TArray<int32> EffectorSolverIndices;

	/** Per-bone settings to control the resulting pose. Includes limits and preferred angles. */
	UPROPERTY(meta = (Input))
	TArray<FPBIKBoneSetting> BoneSettings;

	/** These bones will be excluded from the solver. They will not bend and will not contribute to the constraint set.
	 * Use the ExcludedBones array instead of setting Rotation Stiffness to very high values or Rotation Limits with zero range. */
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	TArray<FName> ExcludedBones;

	/** Global solver settings. */
	UPROPERTY(meta = (Input))
	FPBIKSolverSettings Settings;

	/** Debug drawing options. */
	UPROPERTY(meta = (Input))
	FPBIKDebug Debug;

	/** Runtime-only data */
	UPROPERTY(transient)
	FPBIKWorkData WorkData;
};

#undef UE_API
